{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NNERuntimeORT",
	"Description": "ONNX Runtime backed runtime for the Neural Network Engine (NNE), accelerated by the CPU and DirectML execution providers.",
	"Category": "ML",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://dev.epicgames.com/community/learning/courses/e7w/unreal-engine-neural-network-engine-nne",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/t/course-neural-network-engine-nne/1162628",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "NNERuntimeORT",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "LinuxArm64", "Mac" ]
		}
	]
}