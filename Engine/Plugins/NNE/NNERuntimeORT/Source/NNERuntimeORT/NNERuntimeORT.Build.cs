// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;

public class NNERuntimeORT : ModuleRules
{
	public NNERuntimeORT( ReadOnlyTargetRules Target ) : base( Target )
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"NNE",
			"NNEOnnxruntime",
			"Projects",
			"RenderCore",
			"DeveloperSettings",
			"RHI"
		});

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"D3D12RHI"
			});

			AddEngineThirdPartyPrivateStaticDependencies(Target, new string[]
			{
				"DirectML",
				"DX12"
			});

			// In Engine\Source\ThirdParty\Windows\DX12\DX12.Build.cs, we link against an older version of dxguid.lib, so we can not use the new one.
			// if (Version.TryParse(Target.WindowsPlatform.WindowsSdkVersion, out Version WindowsSdkVersion) && WindowsSdkVersion.Build >= 26100 /* Windows 11, version 24H2 */)
			// {
			// 	PublicDefinitions.Add("ORT_USE_NEW_DXCORE_FEATURES");
			// }
		}

		if (Target.bBuildEditor)
		{
			bEnableExceptions = true;
			bDisableAutoRTFMInstrumentation = true;
		}
	}
}
