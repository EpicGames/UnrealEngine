// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeORT.h"

#include "Misc/SecureHash.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeORTModel.h"
#include "NNERuntimeORTModelFormat.h"
#include "NNERuntimeORTUtils.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#if PLATFORM_WINDOWS
#include "ID3D12DynamicRHI.h"
#endif // PLATFORM_WINDOWS

#include UE_INLINE_GENERATED_CPP_BY_NAME(NNERuntimeORT)

DEFINE_LOG_CATEGORY(LogNNERuntimeORT);

FGuid UNNERuntimeORTCpu::GUID = FGuid((int32)'O', (int32)'C', (int32)'P', (int32)'U');
int32 UNNERuntimeORTCpu::Version = 0x00000004;

namespace UE::NNERuntimeORT::Private::Details
{ 
	//Should be kept in sync with OnnxFileLoaderHelper::InitUNNEModelDataFromFile()
	static FString OnnxExternalDataDescriptorKey(TEXT("OnnxExternalDataDescriptor"));
	static FString OnnxExternalDataBytesKey(TEXT("OnnxExternalDataBytes"));

	FOnnxDataDescriptor MakeOnnxDataDescriptor(TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData)
	{
		FOnnxDataDescriptor OnnxDataDescriptor = {};
		OnnxDataDescriptor.OnnxModelDataSize = FileData.Num();

		if (AdditionalFileData.Contains(OnnxExternalDataDescriptorKey))
		{
			TConstArrayView<uint8> OnnxExternalDataDescriptorBuffer = AdditionalFileData[OnnxExternalDataDescriptorKey];
			FMemoryReaderView OnnxExternalDataDescriptorReader(OnnxExternalDataDescriptorBuffer, /*bIsPersistent = */true);
			TMap<FString, int64> ExternalDataSizes;

			OnnxExternalDataDescriptorReader << ExternalDataSizes;

			int64 CurrentBucketOffset = OnnxDataDescriptor.OnnxModelDataSize;
			for (const auto& Element : ExternalDataSizes)
			{
				const FString DataFilePath = Element.Key;
				FOnnxAdditionalDataDescriptor DataDescriptor;
				DataDescriptor.Path = DataFilePath;
				DataDescriptor.Offset = CurrentBucketOffset;
				DataDescriptor.Size = Element.Value;

				OnnxDataDescriptor.AdditionalDataDescriptors.Emplace(DataDescriptor);
				CurrentBucketOffset += Element.Value;
			}
		}
		return OnnxDataDescriptor;
	}

	void WriteOnnxModelData(FMemoryWriter64 Writer, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData)
	{
		FOnnxDataDescriptor Descriptor = MakeOnnxDataDescriptor(FileData, AdditionalFileData);
		check(FileData.Num() == Descriptor.OnnxModelDataSize);
		Writer << Descriptor;

		Writer.Serialize(const_cast<uint8*>(FileData.GetData()), FileData.Num());

		if (!Descriptor.AdditionalDataDescriptors.IsEmpty())
		{
			

			check(AdditionalFileData.Contains(OnnxExternalDataBytesKey));
			Writer.Serialize(const_cast<uint8*>(AdditionalFileData[OnnxExternalDataBytesKey].GetData()), AdditionalFileData[OnnxExternalDataBytesKey].Num());
		}
	}

	bool IsAvailableGPU(bool bIsDirectMLAvailable)
	{
		static bool bIsD3D12Available = []() -> bool
		{
			return UE::NNERuntimeORT::Private::IsD3D12Available();
		}();
		return bIsDirectMLAvailable && bIsD3D12Available;
	}
	bool IsAvailableRDG(bool bIsDirectMLAvailable)
	{
		static bool bIsRHID3D12Available = []() -> bool
		{
			return UE::NNERuntimeORT::Private::IsRHID3D12Available();
		}();
		return bIsDirectMLAvailable && bIsRHID3D12Available;
	}
} // namespace UE::NNERuntimeORT::Private::Details

UNNERuntimeORTCpu::ECanCreateModelDataStatus UNNERuntimeORTCpu::CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	return (!FileData.IsEmpty() && FileType.Compare("onnx", ESearchCase::IgnoreCase) == 0) ? ECanCreateModelDataStatus::Ok : ECanCreateModelDataStatus::FailFileIdNotSupported;
}

TSharedPtr<UE::NNE::FSharedModelData> UNNERuntimeORTCpu::CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform)
{
	using namespace UE::NNERuntimeORT::Private;

	if (CanCreateModelData(FileType, FileData, AdditionalFileData, FileId, TargetPlatform) != ECanCreateModelDataStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Cannot create the CPU model data with id %ls (Filetype: %ls)", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
		return {};
	}

	TConstArrayView64<uint8> OptimizedModelView = FileData;
	TArray64<uint8> OptimizedModelBuffer;

	//For now only optimize model if there is no external data (as additional data are serialized from the unoptimized model below)
	if (AdditionalFileData.IsEmpty())
	{
		
		if (GraphOptimizationLevel OptimizationLevel = GetGraphOptimizationLevelForCPU(false, IsRunningCookCommandlet()); OptimizationLevel > GraphOptimizationLevel::ORT_DISABLE_ALL)
		{
			TUniquePtr<Ort::SessionOptions> SessionOptions = CreateSessionOptionsDefault(Environment.ToSharedRef());
			SessionOptions->SetGraphOptimizationLevel(OptimizationLevel);
			SessionOptions->EnableCpuMemArena();

			if (!OptimizeModel(Environment.ToSharedRef(), *SessionOptions, FileData, OptimizedModelBuffer))
			{
				UE_LOGF(LogNNERuntimeORT, Error, "Failed to optimize model for CPU with id %ls, model data will not be available", *FileId.ToString(EGuidFormats::Digits).ToLower());
				return {};
			}

			OptimizedModelView = OptimizedModelBuffer;
		}
	}

	TArray64<uint8> Result;
	FMemoryWriter64 Writer(Result, /*bIsPersitent =*/ true);
	Writer << UNNERuntimeORTCpu::GUID;
	Writer << UNNERuntimeORTCpu::Version;

	Details::WriteOnnxModelData(Writer, OptimizedModelView, AdditionalFileData);

	return MakeShared<UE::NNE::FSharedModelData>(MakeSharedBufferFromArray(MoveTemp(Result)), 0);
}

FString UNNERuntimeORTCpu::GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	return FileId.ToString(EGuidFormats::Digits) + "-" + UNNERuntimeORTCpu::GUID.ToString(EGuidFormats::Digits) + "-" + FString::FromInt(UNNERuntimeORTCpu::Version);
}

void UNNERuntimeORTCpu::Init(TSharedRef<UE::NNERuntimeORT::Private::FEnvironment> InEnvironment)
{
	Environment = InEnvironment;
}

FString UNNERuntimeORTCpu::GetRuntimeName() const
{
	return TEXT("NNERuntimeORTCpu");
}

UNNERuntimeORTCpu::ECanCreateModelCPUStatus UNNERuntimeORTCpu::CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const
{
	check(ModelData != nullptr);

	constexpr int32 GuidSize = sizeof(UNNERuntimeORTCpu::GUID);
	constexpr int32 VersionSize = sizeof(UNNERuntimeORTCpu::Version);
	const TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());

	if (!SharedData.IsValid())
	{
		return ECanCreateModelCPUStatus::Fail;
	}

	TConstArrayView64<uint8> Data = SharedData->GetView();

	if (Data.Num() <= GuidSize + VersionSize)
	{
		return ECanCreateModelCPUStatus::Fail;
	}

	bool bResult = FGenericPlatformMemory::Memcmp(&(Data[0]), &(UNNERuntimeORTCpu::GUID), GuidSize) == 0;
	bResult &= FGenericPlatformMemory::Memcmp(&(Data[GuidSize]), &(UNNERuntimeORTCpu::Version), VersionSize) == 0;

	return bResult ? ECanCreateModelCPUStatus::Ok : ECanCreateModelCPUStatus::Fail;
}

TSharedPtr<UE::NNE::IModelCPU> UNNERuntimeORTCpu::CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData)
{
	check(ModelData != nullptr);

	if (CanCreateModelCPU(ModelData) != ECanCreateModelCPUStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Cannot create a CPU model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelCPU>();
	}

	const TSharedRef<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName()).ToSharedRef();

	return MakeShared<UE::NNERuntimeORT::Private::FModelORTCpu>(Environment.ToSharedRef(), SharedData);
}

/*
 * FNNERuntimeORTDmlImpl
 */
class FNNERuntimeORTDmlImpl : public INNERuntimeORTDmlImpl
{

	using ECanCreateModelCommonStatus = UE::NNE::EResultStatus;

private:
	TSharedPtr<UE::NNERuntimeORT::Private::FEnvironment> Environment;

public:
	static FGuid GUID;
	static int32 Version;

	FNNERuntimeORTDmlImpl() = default;
	virtual ~FNNERuntimeORTDmlImpl() = default;

virtual void Init(TSharedRef<UE::NNERuntimeORT::Private::FEnvironment> InEnvironment, bool bInDirectMLAvailable) override
{
	Environment = InEnvironment;
	bIsDirectMLAvailable = bInDirectMLAvailable;
}

virtual FString GetRuntimeName() const override
{
	return TEXT("NNERuntimeORTDml");
}

virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override
{
	return (!FileData.IsEmpty() && FileType.Compare("onnx", ESearchCase::IgnoreCase) == 0) ? ECanCreateModelDataStatus::Ok : ECanCreateModelDataStatus::FailFileIdNotSupported;
}

virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override
{
	using namespace UE::NNERuntimeORT::Private;

	if (CanCreateModelData(FileType, FileData, AdditionalFileData, FileId, TargetPlatform) != ECanCreateModelDataStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Cannot create the Dml model data with id %ls (Filetype: %ls)", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
		return {};
	}

	TConstArrayView64<uint8> OptimizedModelView = FileData;
	TArray64<uint8> OptimizedModelBuffer;

	//For now only optimize model if there is no external data (as additional data are serialized from the unoptimized model below)
	if (AdditionalFileData.IsEmpty())
	{
		if (GraphOptimizationLevel OptimizationLevel = GetGraphOptimizationLevelForDML(false, IsRunningCookCommandlet()); OptimizationLevel > GraphOptimizationLevel::ORT_DISABLE_ALL)
		{
			TUniquePtr<Ort::SessionOptions> SessionOptions = CreateSessionOptionsDefault(Environment.ToSharedRef());
			SessionOptions->SetGraphOptimizationLevel(OptimizationLevel);
			SessionOptions->SetExecutionMode(ExecutionMode::ORT_SEQUENTIAL);
			SessionOptions->DisableMemPattern();

			if (!OptimizeModel(Environment.ToSharedRef(), *SessionOptions, FileData, OptimizedModelBuffer))
			{
				UE_LOGF(LogNNERuntimeORT, Error, "Failed to optimize model for DirectML with id %ls, model data will not be available", *FileId.ToString(EGuidFormats::Digits).ToLower());

				return {};
			}

			OptimizedModelView = OptimizedModelBuffer;
		}
	}

	TArray64<uint8> Result;
	FMemoryWriter64 Writer(Result, /*bIsPersitent =*/ true);
	Writer << GUID;
	Writer << Version;

	Details::WriteOnnxModelData(Writer, OptimizedModelView, AdditionalFileData);

	return MakeShared<UE::NNE::FSharedModelData>(MakeSharedBufferFromArray(MoveTemp(Result)), 0);
}

virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override
{
	return FileId.ToString(EGuidFormats::Digits) + "-" + GUID.ToString(EGuidFormats::Digits) + "-" + FString::FromInt(Version);
}

virtual ECanCreateModelGPUStatus CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const override
{
	if (!ensureMsgf(UE::NNERuntimeORT::Private::Details::IsAvailableGPU(bIsDirectMLAvailable), TEXT("GPU interface should not be available!")))
	{
		return ECanCreateModelCommonStatus::Fail;
	}

	return CanCreateModelCommon(ModelData, false) == ECanCreateModelCommonStatus::Ok ? ECanCreateModelGPUStatus::Ok : ECanCreateModelGPUStatus::Fail;
}

virtual TSharedPtr<UE::NNE::IModelGPU> CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) override
{
#if PLATFORM_WINDOWS
	check(ModelData);

	if (CanCreateModelGPU(ModelData) != ECanCreateModelGPUStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Cannot create a GPU model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return {};
	}

	const TSharedRef<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName()).ToSharedRef();

	return MakeShared<UE::NNERuntimeORT::Private::FModelORTDmlGPU>(Environment.ToSharedRef(), SharedData);
#else // PLATFORM_WINDOWS
	return TSharedPtr<UE::NNE::IModelGPU>();
#endif // PLATFORM_WINDOWS
}

virtual ECanCreateModelRDGStatus CanCreateModelRDG(TObjectPtr<UNNEModelData> ModelData) const override
{
	if (!ensureMsgf(UE::NNERuntimeORT::Private::Details::IsAvailableRDG(bIsDirectMLAvailable), TEXT("RDG interface should not be available!")))
	{
		return ECanCreateModelCommonStatus::Fail;
	}

	return CanCreateModelCommon(ModelData) == ECanCreateModelCommonStatus::Ok ? ECanCreateModelRDGStatus::Ok : ECanCreateModelRDGStatus::Fail;
}

virtual TSharedPtr<UE::NNE::IModelRDG> CreateModelRDG(TObjectPtr<UNNEModelData> ModelData) override
{
#if PLATFORM_WINDOWS
	check(ModelData);

	if (CanCreateModelRDG(ModelData) != ECanCreateModelRDGStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Cannot create a RDG model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return {};
	}

	const TSharedRef<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName()).ToSharedRef();

	return MakeShared<UE::NNERuntimeORT::Private::FModelORTDmlRDG>(Environment.ToSharedRef(), SharedData);
#else // PLATFORM_WINDOWS
	return {};
#endif // PLATFORM_WINDOWS
}

private:
ECanCreateModelCommonStatus CanCreateModelCommon(const TObjectPtr<UNNEModelData> ModelData, bool bRHID3D12Required = true) const
{
#if PLATFORM_WINDOWS
	check(ModelData != nullptr);

	constexpr int32 GuidSize = sizeof(GUID);
	constexpr int32 VersionSize = sizeof(Version);
	const TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());

	if (!SharedData.IsValid())
	{
		return ECanCreateModelCommonStatus::Fail;
	}

	TConstArrayView64<uint8> Data = SharedData->GetView();

	if (Data.Num() <= GuidSize + VersionSize)
	{
		return ECanCreateModelCommonStatus::Fail;
	}

	static const FGuid DeprecatedGUID = FGuid((int32)'O', (int32)'G', (int32)'P', (int32)'U');

	bool bResult = FGenericPlatformMemory::Memcmp(&(Data[0]), &(GUID), GuidSize) == 0;
	bResult |= FGenericPlatformMemory::Memcmp(&(Data[0]), &(DeprecatedGUID), GuidSize) == 0;
	bResult &= FGenericPlatformMemory::Memcmp(&(Data[GuidSize]), &(Version), VersionSize) == 0;

	return bResult ? ECanCreateModelCommonStatus::Ok : ECanCreateModelCommonStatus::Fail;
#else // PLATFORM_WINDOWS
	return ECanCreateModelCommonStatus::Fail;
#endif // PLATFORM_WINDOWS
}

	bool bIsDirectMLAvailable = false;
};

FGuid FNNERuntimeORTDmlImpl::GUID = FGuid((int32)'O', (int32)'D', (int32)'M', (int32)'L');
int32 FNNERuntimeORTDmlImpl::Version = 0x00000004;

UNNERuntimeORTDmlProxy::UNNERuntimeORTDmlProxy()
{
	Impl = MakeUnique<FNNERuntimeORTDmlImpl>();
}

void UNNERuntimeORTDmlProxy::Init(TSharedRef<UE::NNERuntimeORT::Private::FEnvironment> InEnvironment, bool bInDirectMLAvailable) { Impl->Init(InEnvironment, bInDirectMLAvailable); }

FString UNNERuntimeORTDmlProxy::GetRuntimeName() const { return Impl->GetRuntimeName(); }

UNNERuntimeORTDmlProxy::ECanCreateModelDataStatus UNNERuntimeORTDmlProxy::CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const { return Impl->CanCreateModelData(FileType, FileData, AdditionalFileData, FileId, TargetPlatform); }
TSharedPtr<UE::NNE::FSharedModelData> UNNERuntimeORTDmlProxy::CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) { return Impl->CreateModelData(FileType, FileData, AdditionalFileData, FileId, TargetPlatform); }
FString UNNERuntimeORTDmlProxy::GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const { return Impl->GetModelDataIdentifier(FileType, FileData, AdditionalFileData, FileId, TargetPlatform); }

namespace UE::NNERuntimeORT::Private
{

TWeakObjectPtr<UNNERuntimeORTDmlProxy> MakeRuntimeDml(bool bDirectMLAvailable)
{
	// Important note: the check for GPU interface availability might be expensive, e.g. on Vulkan, because we need to check for D3D12 support.
	// That will cause the DLL to load and perform other expensive work etc. This is undesired in the module loading phase and specifically
	// we don't know whether its actually ever used.
	// 
	// Therefore, at this moment we exceptionally assume D3D12 "is present" and will do the check later in CanCreateModelGPU()
	const bool bRHID3D12Available = IsRHID3D12Available();

	const bool bIsAvailableRDG = bDirectMLAvailable && bRHID3D12Available;

	UE_LOGF(LogNNERuntimeORT, Log, "MakeRuntimeORTDml:");
	UE_LOGF(LogNNERuntimeORT, Log, "  DirectML:  %ls", (bDirectMLAvailable ? TEXT("yes") : TEXT("no")));
	UE_LOGF(LogNNERuntimeORT, Log, "  RHI D3D12: %ls", (bRHID3D12Available ? TEXT("yes") : TEXT("no")));
	UE_LOGF(LogNNERuntimeORT, Log, "  D3D12:     n/a (checked in CanCreateModelGPU)");

	UE_LOGF(LogNNERuntimeORT, Log, "Interface availability:");
	UE_LOGF(LogNNERuntimeORT, Log, "  GPU: n/a (checked in CanCreateModelGPU)");
	UE_LOGF(LogNNERuntimeORT, Log, "  RDG: %ls", (bIsAvailableRDG ? TEXT("yes") : TEXT("no")));

	if (bIsAvailableRDG) return NewObject<UNNERuntimeORTDml_GPU_RDG>();
	else if (bDirectMLAvailable) return NewObject<UNNERuntimeORTDml_GPU>();

#if WITH_EDITOR
	UE_LOGF(LogNNERuntimeORT, Log, "NNERuntimeORTDml can only cook!");
	return NewObject<UNNERuntimeORTDmlProxy>();
#else
	UE_LOGF(LogNNERuntimeORT, Log, "NNERuntimeORTDml is not available!");
	return {};
#endif
}

} // namespace UE::NNERuntimeORT::Private