// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntime.h"
#include "NNERuntimeCPU.h"
#include "NNERuntimeGPU.h"
#include "NNERuntimeRDG.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectBaseUtility.h"

#include "NNERuntimeORT.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogNNERuntimeORT, Log, All);

namespace UE::NNERuntimeORT::Private
{
	class FEnvironment;
}

UCLASS()
class UNNERuntimeORTCpu : public UObject, public INNERuntime, public INNERuntimeCPU
{
	GENERATED_BODY()

private:
	TSharedPtr<UE::NNERuntimeORT::Private::FEnvironment> Environment;

public:
	static FGuid GUID;
	static int32 Version;

	virtual ~UNNERuntimeORTCpu() = default;

	void Init(TSharedRef<UE::NNERuntimeORT::Private::FEnvironment> InEnvironment);

	virtual FString GetRuntimeName() const override;
	
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override;
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;

	virtual ECanCreateModelCPUStatus CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const override;
	virtual TSharedPtr<UE::NNE::IModelCPU> CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) override;
};

class INNERuntimeORTDmlImpl : public INNERuntime, public INNERuntimeGPU, public INNERuntimeRDG
{
public:
	virtual void Init(TSharedRef<UE::NNERuntimeORT::Private::FEnvironment> InEnvironment, bool bInDirectMLAvailable) = 0;
};

UCLASS()
class UNNERuntimeORTDmlProxy : public UObject, public INNERuntime
{
	GENERATED_BODY()

public:
	UNNERuntimeORTDmlProxy();
	~UNNERuntimeORTDmlProxy() = default;
	
	void Init(TSharedRef<UE::NNERuntimeORT::Private::FEnvironment> InEnvironment, bool bInDirectMLAvailable);

	virtual FString GetRuntimeName() const override;

	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override;
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;

protected:
	TUniquePtr<INNERuntimeORTDmlImpl> Impl;
};

UCLASS()
class UNNERuntimeORTDml_GPU_RDG : public UNNERuntimeORTDmlProxy, public INNERuntimeGPU, public INNERuntimeRDG
{
	GENERATED_BODY()

public:
	virtual ECanCreateModelGPUStatus CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const override { return Impl->CanCreateModelGPU(ModelData); }
	virtual TSharedPtr<UE::NNE::IModelGPU> CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) override { return Impl->CreateModelGPU(ModelData); }

	virtual ECanCreateModelRDGStatus CanCreateModelRDG(TObjectPtr<UNNEModelData> ModelData) const override { return Impl->CanCreateModelRDG(ModelData); }
	virtual TSharedPtr<UE::NNE::IModelRDG> CreateModelRDG(TObjectPtr<UNNEModelData> ModelData) override { return Impl->CreateModelRDG(ModelData); }
};

UCLASS()
class UNNERuntimeORTDml_GPU : public UNNERuntimeORTDmlProxy, public INNERuntimeGPU
{
	GENERATED_BODY()

public:
	virtual ECanCreateModelGPUStatus CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const override { return Impl->CanCreateModelGPU(ModelData); }
	virtual TSharedPtr<UE::NNE::IModelGPU> CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) override { return Impl->CreateModelGPU(ModelData); }
};

namespace UE::NNERuntimeORT::Private
{

TWeakObjectPtr<UNNERuntimeORTDmlProxy> MakeRuntimeDml(bool bDirectMLAvailable);

} // namespace UE::NNERuntimeORT::Private