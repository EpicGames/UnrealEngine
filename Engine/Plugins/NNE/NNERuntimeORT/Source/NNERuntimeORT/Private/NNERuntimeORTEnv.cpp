// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeORTEnv.h"
#include "Misc/ScopeLock.h"

namespace UE::NNERuntimeORT::Private
{

void FEnvironment::Configure(const FConfig& InConfig)
{
	FScopeLock ScopeLock(&CriticalSection);

#if WITH_EDITOR
	// For reloading need to make sure OrtApi::ReleaseEnv() is called before we create a new Ort::Env!
	OrtEnvironment = Ort::Env(nullptr);
#endif // WITH_EDITOR

	checkf(OrtEnvironment == Ort::Env(nullptr), TEXT("Ort::Env already created!"));

	Config = InConfig;
}

const Ort::Env& FEnvironment::GetOrtEnv() const
{
	FScopeLock ScopeLock(&CriticalSection);
	
	// We defer ORT Environment creation to first usage
	if (OrtEnvironment == Ort::Env(nullptr))
	{
		CreateOrtEnv();
	}

	return OrtEnvironment;
}

void FEnvironment::CreateOrtEnv() const
{
	checkf(OrtEnvironment == Ort::Env(nullptr), TEXT("Ort::Env already created!"));

	if (Config.bUseGlobalThreadPool)
	{
		Ort::ThreadingOptions ThreadingOptions;
		ThreadingOptions.SetGlobalIntraOpNumThreads(Config.IntraOpNumThreads);
		ThreadingOptions.SetGlobalInterOpNumThreads(Config.InterOpNumThreads);

		// Calls OrtApi::CreateEnvWithGlobalThreadPools(), needs to be called in conjunction
		// with OrtApi::DisablePerSessionThreads or else the session will use its own thread pools.
		OrtEnvironment = Ort::Env(ThreadingOptions);
	}
	else
	{
		// Calls OrtApi::CreateEnv()
		OrtEnvironment = Ort::Env();
	}

	// 
	// WITH_EDITOR After this point, if we created a new Ort::Env, all Onnxruntime structures (ideally) need to be recreated!
	// At least SessionOption and Session...
	// 
}

} // UE::NNERuntimeORT::Private