// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEOnnxruntime.h"
#include "HAL/CriticalSection.h"

namespace UE::NNERuntimeORT::Private
{

class FEnvironment
{
public:
	struct FConfig
	{
		bool bUseGlobalThreadPool = false;
		int32 IntraOpNumThreads = 0;
		int32 InterOpNumThreads = 0;
	};

	FEnvironment() = default;
	
	~FEnvironment() = default;

	void Configure(const FConfig& InConfig);

	const Ort::Env& GetOrtEnv() const;

	FConfig GetConfig() const { return Config; }

private:
	void CreateOrtEnv() const;

	FConfig Config{};
	mutable Ort::Env OrtEnvironment{nullptr};

	mutable FCriticalSection CriticalSection;
};

} // UE::NNERuntimeORT::Private