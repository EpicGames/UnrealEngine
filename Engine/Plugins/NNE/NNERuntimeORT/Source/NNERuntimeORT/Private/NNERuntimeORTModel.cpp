// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeORTModel.h"

#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMisc.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "NNERuntimeORT.h"
#include "NNERuntimeORTModelFormat.h"
#include "NNERuntimeORTSettings.h"
#include "NNERuntimeORTUtils.h"
#include "RenderGraphUtils.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "Serialization/MemoryReader.h"

#if PLATFORM_WINDOWS
#include "ID3D12DynamicRHI.h"
#endif // PLATFORM_WINDOWS

// DirectML is implemented using COM on all platforms
#ifdef IID_GRAPHICS_PPV_ARGS
#define DML_PPV_ARGS(x) __uuidof(*x), IID_PPV_ARGS_Helper(x)
#else
#define DML_PPV_ARGS(x) IID_PPV_ARGS(x)
#endif

BEGIN_SHADER_PARAMETER_STRUCT(FORTModelInstanceRDGParameters, )
	RDG_BUFFER_ACCESS_ARRAY(InputBuffers)
	RDG_BUFFER_ACCESS_ARRAY(OutputBuffers)
END_SHADER_PARAMETER_STRUCT()

DECLARE_GPU_STAT_NAMED(FNNERuntimeORTDmlRDG, TEXT("FModelInstanceORTDmlRDG::EnqueueRDG"));

namespace UE::NNERuntimeORT::Private
{

namespace Detail
{

	FRuntimeConf MakeRuntimeConfigFromSettings(const UNNERuntimeORTSettings* Settings)
	{
		FRuntimeConf Result{};
#if WITH_EDITOR
			FThreadingOptions ThreadingOptions = Settings->EditorThreadingOptions;
#else
			FThreadingOptions ThreadingOptions = Settings->GameThreadingOptions;
#endif
		Result.ExecutionMode = ThreadingOptions.ExecutionMode == EExecutionMode::SEQUENTIAL ? ExecutionMode::ORT_SEQUENTIAL : ExecutionMode::ORT_PARALLEL;

		return Result;
	}

	FString CreateTempDirPath(const FString& BasePath)
	{
		FString UniqueDirName;
		do
		{
			UniqueDirName = FPaths::Combine(BasePath, *FString::Printf(TEXT("ORTModel_%s"), *FGuid::NewGuid().ToString()));
		} while (IFileManager::Get().DirectoryExists(*UniqueDirName));

		return UniqueDirName;
	}

	bool CreateSession(
		TConstArrayView64<uint8> ModelData,
		const Ort::SessionOptions& SessionOptions,
		const FEnvironment& Environment,
		TUniquePtr<Ort::Session>& Session, FString& TempDirForModelWithExternalData)
	{
		FMemoryReaderView Reader(MakeMemoryView(ModelData), /*bIsPersitent =*/ true);
		FGuid GUID;
		int32 Version;
		Reader << GUID;
		Reader << Version;

		FOnnxDataDescriptor Descriptor;
		Reader << Descriptor;

		int64 BaseDataOffset = Reader.Tell();
		TConstArrayView64<uint8> ModelBuffer = TConstArrayView64<uint8>(&(ModelData.GetData()[BaseDataOffset]), Descriptor.OnnxModelDataSize);

		if (ModelBuffer.Num() == 0)
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Cannot create ORT session: Input model data is empty.");
			return false;
		}

		// Starting with ORT v18 we will get AddExternalInitializersFromFilesInMemory() via onnxruntime_c_api.h
		// however for now we use temp files when working with model with external data.
		if (Descriptor.AdditionalDataDescriptors.Num() > 0)
		{
			FString Filepath;
			if (TempDirForModelWithExternalData.IsEmpty())
			{
				FString ProjIntermediateDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir());
				TempDirForModelWithExternalData = Detail::CreateTempDirPath(ProjIntermediateDir);
				Filepath = FPaths::Combine(TempDirForModelWithExternalData, TEXT("OnnxModel.onnx"));

				// Note: SaveArrayToFile() will create the needed folders as needed both for the Onnx model and the additional data files.
				if (!FFileHelper::SaveArrayToFile(ModelBuffer, *Filepath))
				{
					IFileManager::Get().DeleteDirectory(*TempDirForModelWithExternalData, false, true);
					UE_LOGF(LogNNERuntimeORT, Error, "Large models are an experimental feature at the moment. Could not write model to disk at %ls.", *Filepath);
					return false;
				}

				for (const FOnnxAdditionalDataDescriptor& AdditionalDataDescriptor : Descriptor.AdditionalDataDescriptors)
				{
					FString AdditionalDataFilename = FPaths::Combine(TempDirForModelWithExternalData, *AdditionalDataDescriptor.Path);
					TConstArrayView64<uint8> AdditionalDataBuffer = TConstArrayView64<uint8>(&(ModelData.GetData()[BaseDataOffset + AdditionalDataDescriptor.Offset]), AdditionalDataDescriptor.Size);

					if (!FFileHelper::SaveArrayToFile(AdditionalDataBuffer, *AdditionalDataFilename))
					{
						IFileManager::Get().DeleteDirectory(*TempDirForModelWithExternalData, false, true);
						UE_LOGF(LogNNERuntimeORT, Error, "Large models are an experimental feature at the moment. Could not write additional data to disk at %ls.", *AdditionalDataFilename);
						return false;
					}
				}
			}
			else
			{
				Filepath = FPaths::Combine(TempDirForModelWithExternalData, TEXT("OnnxModel.onnx"));
			}

			Session = CreateOrtSession(Environment, Filepath, SessionOptions);
		}
		else
		{
			Session = CreateOrtSessionFromArray(Environment, ModelBuffer, SessionOptions);
		}

		return Session.IsValid();
	}

	bool SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes, TConstArrayView<NNE::FTensorDesc> InputSymbolicTensors, TArray<NNE::FTensorShape> &InputTensorShapes)
	{
		InputTensorShapes.Reset(InInputShapes.Num());

		if (InInputShapes.Num() != InputSymbolicTensors.Num())
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Number of input shapes does not match number of input tensors");
			return false;
		}

		for (int32 i = 0; i < InInputShapes.Num(); ++i)
		{
			const NNE::FTensorDesc SymbolicDesc = InputSymbolicTensors[i];
			if (!InInputShapes[i].IsCompatibleWith(SymbolicDesc.GetShape()))
			{
				UE_LOGF(LogNNERuntimeORT, Error, "Input shape does not match input tensor %ls of index %d", *SymbolicDesc.GetName(), i);
				return false;
			}
		}

		InputTensorShapes = InInputShapes;

		//Implementations are responsible to handle output and intermediate tensor shape inference.
		//This base implementation only validate that all inputs are matching what the model can support.
		return true;
	}

} // namespace Detail

template <class T>
TConstArrayView<NNE::FTensorDesc> FModelInstanceORTBase<T>::GetInputTensorDescs() const
{
	return InputSymbolicTensors;
}

template <class T>
TConstArrayView<NNE::FTensorDesc> FModelInstanceORTBase<T>::GetOutputTensorDescs() const
{
	return OutputSymbolicTensors;
}

template <class T>
TConstArrayView<NNE::FTensorShape> FModelInstanceORTBase<T>::GetInputTensorShapes() const
{
	return InputTensorShapes;
}

template <class T>
TConstArrayView<NNE::FTensorShape> FModelInstanceORTBase<T>::GetOutputTensorShapes() const
{
	return OutputTensorShapes;
}

template <class T>
typename T::ESetInputTensorShapesStatus FModelInstanceORTBase<T>::SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes)
{
	if (Detail::SetInputTensorShapes(InInputShapes, InputSymbolicTensors, InputTensorShapes))
	{
		return T::ESetInputTensorShapesStatus::Ok;
	}

	return T::ESetInputTensorShapesStatus::Fail;
}

template <class ModelInterface, class TensorBinding> 
FModelInstanceORTRunSync<ModelInterface, TensorBinding>::FModelInstanceORTRunSync(const FRuntimeConf& InRuntimeConf, TSharedRef<FEnvironment> InEnvironment)
	: RuntimeConf(InRuntimeConf), Environment(InEnvironment)
{

}

template <class ModelInterface, class TensorBinding>
FModelInstanceORTRunSync<ModelInterface, TensorBinding>::~FModelInstanceORTRunSync()
{
	Session.Reset();
	if (!TempDirForModelWithExternalData.IsEmpty())
	{
		if (!IFileManager::Get().DeleteDirectory(*TempDirForModelWithExternalData, false, true))
		{
			UE_LOGF(LogNNERuntimeORT, Warning, "Large models are an experimental feature at the moment. Could not delete temp directy %ls on model instance destruction.", *TempDirForModelWithExternalData);
		}
	}
}

template <class ModelInterface, class TensorBinding> 
bool FModelInstanceORTRunSync<ModelInterface, TensorBinding>::Init(TConstArrayView64<uint8> ModelData)
{
#if WITH_EDITOR
	try
#endif // WITH_EDITOR
	{
		if (!InitializedAndConfigureMembers())
		{
			UE_LOGF(LogNNERuntimeORT, Error, "InitializedAndConfigureMembers failed.");
			return false;
		}

		if (!Detail::CreateSession(ModelData, *SessionOptions, *Environment, Session, TempDirForModelWithExternalData))
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Session creation failed.");
			return false;
		}

		if (!ConfigureTensors(true))
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Failed to configure Inputs tensors.");
			return false;
		}
		if (!ConfigureTensors(false))
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Failed to configure Outputs tensors.");
			return false;
		}
	}
#if WITH_EDITOR
	catch (const Ort::Exception& Exception)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "%ls", UTF8_TO_TCHAR(Exception.what()));
		return false;
	}
	catch (...)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Unknown exception!");
		return false;
	}
#endif // WITH_EDITOR

	return true;
}

template <class ModelInterface, class TensorBinding> 
bool FModelInstanceORTRunSync<ModelInterface, TensorBinding>::InitializedAndConfigureMembers()
{
	Allocator = MakeUnique<Ort::AllocatorWithDefaultOptions>();
	MemoryInfo = MakeUnique<Ort::MemoryInfo>(Ort::MemoryInfo::CreateCpu(OrtDeviceAllocator, OrtMemTypeCPU));

	return true;
}

template <class ModelInterface, class TensorBinding>
bool FModelInstanceORTRunSync<ModelInterface, TensorBinding>::ConfigureTensors(bool bAreTensorInputs)
{
	const uint32 NumberTensors							= bAreTensorInputs ? Session->GetInputCount() : Session->GetOutputCount();
	TArray<NNE::FTensorDesc>& SymbolicTensorDescs		= bAreTensorInputs ? FModelInstanceORTBase<ModelInterface>::InputSymbolicTensors : FModelInstanceORTBase<ModelInterface>::OutputSymbolicTensors;
	TArray<ONNXTensorElementDataType>& TensorsORTType	= bAreTensorInputs ? InputTensorsORTType	: OutputTensorsORTType;
	TArray<char*>& TensorNames							= bAreTensorInputs ? InputTensorNames		: OutputTensorNames;
	TArray<Ort::AllocatedStringPtr>& TensorNameValues	= bAreTensorInputs ? InputTensorNameValues	: OutputTensorNameValues;

	SymbolicTensorDescs.Reset();

	for (uint32 TensorIndex = 0; TensorIndex < NumberTensors; ++TensorIndex)
	{
		// Get Tensor name
		Ort::AllocatedStringPtr CurTensorName = bAreTensorInputs ? Session->GetInputNameAllocated(TensorIndex, *Allocator) : Session->GetOutputNameAllocated(TensorIndex, *Allocator);
		TensorNameValues.Emplace(MoveTemp(CurTensorName));
		TensorNames.Emplace(TensorNameValues.Last().get());

		// Get node type
		const Ort::TypeInfo CurrentTypeInfo = bAreTensorInputs ? Session->GetInputTypeInfo(TensorIndex) : Session->GetOutputTypeInfo(TensorIndex);
		const Ort::ConstTensorTypeAndShapeInfo CurrentTensorInfo = CurrentTypeInfo.GetTensorTypeAndShapeInfo();
		const ONNXTensorElementDataType ONNXTensorElementDataTypeEnum = CurrentTensorInfo.GetElementType();
		const TypeInfoORT TypeInfo = TranslateTensorTypeORTToNNE(ONNXTensorElementDataTypeEnum);

		TensorsORTType.Emplace(ONNXTensorElementDataTypeEnum);

		// Get shape
		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> ShapeData;
		ShapeData.Reserve(CurrentTensorInfo.GetShape().size());
		for (int64 CurrentTensorSize : CurrentTensorInfo.GetShape())
		{
			ShapeData.Add((int32)CurrentTensorSize);
		}

		const NNE::FSymbolicTensorShape Shape = NNE::FSymbolicTensorShape::Make(ShapeData);
		const NNE::FTensorDesc SymbolicTensorDesc = NNE::FTensorDesc::Make(FString(TensorNames.Last()), Shape, TypeInfo.DataType);

		check(SymbolicTensorDesc.GetElementByteSize() == TypeInfo.ElementSize);
		SymbolicTensorDescs.Emplace(SymbolicTensorDesc);
	}

	return true;
}

template <class ModelInterface, class TensorBinding> 
typename ModelInterface::ESetInputTensorShapesStatus FModelInstanceORTRunSync<ModelInterface, TensorBinding>::SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes)
{
	using ModelInstanceBase = FModelInstanceORTBase<ModelInterface>;

	InputTensors.Reset();
	OutputTensors.Reset();
	ModelInstanceBase::OutputTensorShapes.Reset();

	// Verify input shape are valid for the model and set InputTensorShapes
	if (typename ModelInterface::ESetInputTensorShapesStatus Status = ModelInstanceBase::SetInputTensorShapes(InInputShapes); Status != ModelInterface::ESetInputTensorShapesStatus::Ok)
	{
		return Status;
	}

	// Setup concrete input tensor
	for (int32 i = 0; i < ModelInstanceBase::InputSymbolicTensors.Num(); ++i)
	{
		FTensor Tensor = FTensor::Make(InInputShapes[i], ModelInstanceBase::InputSymbolicTensors[i].GetDataType());
		InputTensors.Emplace(Tensor);
	}

	// Setup concrete output shapes only if all model output shapes are concretes, otherwise it will be set during Run()
	for (NNE::FTensorDesc SymbolicTensorDesc : ModelInstanceBase::OutputSymbolicTensors)
	{
		if (SymbolicTensorDesc.GetShape().IsConcrete())
		{
			FTensor Tensor = FTensor::MakeFromSymbolicDesc(SymbolicTensorDesc);
			OutputTensors.Emplace(Tensor);
			ModelInstanceBase::OutputTensorShapes.Emplace(Tensor.GetShape());
		}
	}
	if (OutputTensors.Num() != ModelInstanceBase::OutputSymbolicTensors.Num())
	{
		OutputTensors.Reset();
		ModelInstanceBase::OutputTensorShapes.Reset();
	}

	return ModelInterface::ESetInputTensorShapesStatus::Ok;
}

template <class TensorBinding>
Ort::Value CreateTensor(const Ort::MemoryInfo& MemoryInfo, const TensorBinding& Binding, const FTensor& Tensor, const ONNXTensorElementDataType ElementDataType)
{
	const uint64 SizeInBytes = Tensor.GetDataSize();
	const uint32 ShapeLen = (uint32)Tensor.GetShape().Rank();

	TUniquePtr<int64_t[]> Shape = MakeUnique<int64_t[]>(Tensor.GetShape().Rank());
	for (int32 DimIndex = 0; DimIndex < Tensor.GetShape().Rank(); ++DimIndex)
	{
		Shape.Get()[DimIndex] = Tensor.GetShape().GetData()[DimIndex];
	}
	
	return	Ort::Value::CreateTensor(MemoryInfo, Binding.Data, SizeInBytes, Shape.Get(), ShapeLen, ElementDataType);
}

template <class ModelInterface, class TensorBinding>
typename ModelInterface::ERunSyncStatus FModelInstanceORTRunSync<ModelInterface, TensorBinding>::RunSync(TConstArrayView<TensorBinding> InInputBindings, TConstArrayView<TensorBinding> InOutputBindings)
{
	SCOPED_NAMED_EVENT_TEXT("FModelInstanceORTRunSync::RunSync", FColor::Magenta);

	if (!Session.IsValid())
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Called without a Session.");
		return ModelInterface::ERunSyncStatus::Fail;
	}

	// Verify the model inputs were prepared
	if (FModelInstanceORTBase<ModelInterface>::InputTensorShapes.IsEmpty())
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Input shapes are not set, please call SetInputTensorShapes.");
		return ModelInterface::ERunSyncStatus::Fail;
	}

	check(FModelInstanceORTBase<ModelInterface>::InputTensorShapes.Num() == InputTensors.Num());
	check(FModelInstanceORTBase<ModelInterface>::InputTensorShapes.Num() == InputTensorNames.Num());
	check(FModelInstanceORTBase<ModelInterface>::InputSymbolicTensors.Num() == InputTensors.Num());

	if (InInputBindings.Num() != InputTensors.Num())
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Input bindings need to match input tensor descriptor count (got %d, expected %d).", InInputBindings.Num(), InputTensors.Num());
		return ModelInterface::ERunSyncStatus::Fail;
	}

	check(FModelInstanceORTBase<ModelInterface>::OutputSymbolicTensors.Num() == OutputTensorNames.Num());

	if (!InOutputBindings.IsEmpty() && InOutputBindings.Num() != OutputTensorNames.Num())
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Output binding can be empty or needs to match output tensor descriptor count (got %d, expected %d).", InOutputBindings.Num(), OutputTensorNames.Num());
		return ModelInterface::ERunSyncStatus::Fail;
	}

#if WITH_EDITOR
	try
#endif // WITH_EDITOR
	{
		TArray<Ort::Value> OrtInputTensors;
		for (int32 i = 0; i < InputTensorNames.Num(); i++)
		{
			const TensorBinding& Binding = InInputBindings[i];
			const FTensor& Tensor = InputTensors[i];

			if (!Binding.Data && Binding.SizeInBytes != 0)
			{
				UE_LOG(LogNNERuntimeORT, Error, TEXT("Binding input tensor %d is not set but given size is non-zero %" UINT64_FMT "."), i, Binding.SizeInBytes);
				return ModelInterface::ERunSyncStatus::Fail;
			}

			if (Binding.SizeInBytes != Tensor.GetDataSize())
			{
				UE_LOG(LogNNERuntimeORT, Error, TEXT("Binding input tensor %d size does not match size given by tensor descriptor (got %" UINT64_FMT ", expected %" UINT64_FMT ")."), i, Binding.SizeInBytes, Tensor.GetDataSize());
				return ModelInterface::ERunSyncStatus::Fail;
			}

			OrtInputTensors.Add(CreateTensor(*MemoryInfo, Binding, Tensor, InputTensorsORTType[i]));
		}

		TArray<Ort::Value> OrtOutputTensors;
		for (int32 i = 0; i < OutputTensorNames.Num(); i++)
		{
			if (OutputTensors.IsEmpty() ||
				InOutputBindings.IsEmpty() ||
				!InOutputBindings[i].Data ||
				InOutputBindings[i].SizeInBytes < OutputTensors[i].GetDataSize())
			{
				OrtOutputTensors.Emplace(nullptr);
			}
			else
			{
				OrtOutputTensors.Add(CreateTensor(*MemoryInfo, InOutputBindings[i], OutputTensors[i], OutputTensorsORTType[i]));
			}
		}

		Session->Run(Ort::RunOptions{nullptr},
			InputTensorNames.GetData(), &OrtInputTensors[0], InputTensorNames.Num(),
			OutputTensorNames.GetData(), &OrtOutputTensors[0], OutputTensorNames.Num());

		// At this (latest) stage shapes are known, therefore set them if not present yet and possibly copy data to output binding
		if (OutputTensors.IsEmpty())
		{
			check(FModelInstanceORTBase<ModelInterface>::OutputTensorShapes.IsEmpty());

			for (int32 i = 0; i < OutputTensorNames.Num(); i++)
			{
				const NNE::FTensorDesc& TensorDesc = FModelInstanceORTBase<ModelInterface>::OutputSymbolicTensors[i];
				const NNE::FTensorShape Shape = NNE::FTensorShape::Make(OrtHelper::GetShape(OrtOutputTensors[i]));
				const FTensor Tensor = FTensor::Make(Shape, TensorDesc.GetDataType());

				OutputTensors.Add(Tensor);
				FModelInstanceORTBase<ModelInterface>::OutputTensorShapes.Add(Shape);

				if (!InOutputBindings.IsEmpty() &&
					InOutputBindings[i].Data &&
					Tensor.GetDataSize() > 0 &&
					InOutputBindings[i].SizeInBytes >= Tensor.GetDataSize())
				{
					FMemory::Memcpy(InOutputBindings[i].Data, OrtOutputTensors[i].GetTensorData<void>(), Tensor.GetDataSize());
				}
			}
		}
	}
#if WITH_EDITOR
	catch (const Ort::Exception& Exception)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "%ls", UTF8_TO_TCHAR(Exception.what()));
		return ModelInterface::ERunSyncStatus::Fail;
	}
	catch (...)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Unknown exception!");
		return ModelInterface::ERunSyncStatus::Fail;
	}
#endif // WITH_EDITOR

	return ModelInterface::ERunSyncStatus::Ok;
}

TSharedPtr<NNE::IModelInstanceCPU> FModelORTCpu::CreateModelInstanceCPU()
{
	const FRuntimeConf RuntimeConfig = Detail::MakeRuntimeConfigFromSettings(GetDefault<UNNERuntimeORTSettings>());

	TSharedPtr<FModelInstanceORTCpu> ModelInstance = MakeShared<FModelInstanceORTCpu>(RuntimeConfig, Environment);
	if (!ModelInstance->Init(ModelData->GetView()))
	{
		return {};
	}

	return ModelInstance;
}

FModelORTCpu::FModelORTCpu(TSharedRef<FEnvironment> InEnvironment, TSharedRef<UE::NNE::FSharedModelData> InModelData) :
	Environment(InEnvironment), ModelData(InModelData)
{
}

bool FModelInstanceORTCpu::InitializedAndConfigureMembers()
{
	if (!FModelInstanceORTRunSync::InitializedAndConfigureMembers())
	{
		return false;
	}

	SessionOptions = CreateSessionOptionsDefault(Environment);
	if (!SessionOptions.IsValid())
	{
		return false;
	}

	SessionOptions->SetExecutionMode(RuntimeConf.ExecutionMode);
	SessionOptions->SetGraphOptimizationLevel(GetGraphOptimizationLevelForCPU(true));
	SessionOptions->EnableCpuMemArena();

	return true;
}

#if PLATFORM_WINDOWS
TSharedPtr<NNE::IModelInstanceGPU> FModelORTDmlGPU::CreateModelInstanceGPU()
{
	const FRuntimeConf RuntimeConfig = Detail::MakeRuntimeConfigFromSettings(GetDefault<UNNERuntimeORTSettings>());

	TSharedPtr<FModelInstanceORTDmlGPU> ModelInstance = MakeShared<FModelInstanceORTDmlGPU>(RuntimeConfig, Environment);
	if (!ModelInstance->Init(ModelData->GetView()))
	{
		return {};
	}

	return ModelInstance;
}

FModelORTDmlGPU::FModelORTDmlGPU(TSharedRef<FEnvironment> InEnvironment, TSharedRef<UE::NNE::FSharedModelData> InModelData) :
	Environment(InEnvironment), ModelData(InModelData)
{
}

bool FModelInstanceORTDmlGPU::InitializedAndConfigureMembers()
{
	if (!FModelInstanceORTRunSync::InitializedAndConfigureMembers())
	{
		return false;
	}

	SessionOptions = CreateSessionOptionsForDirectML(Environment, false);
	if (!SessionOptions.IsValid())
	{
		return false;
	}

	SessionOptions->SetGraphOptimizationLevel(GetGraphOptimizationLevelForDML(true));

	return true;
}

FModelORTDmlRDG::FModelORTDmlRDG(TSharedRef<FEnvironment> InEnvironment, TSharedRef<UE::NNE::FSharedModelData> InModelData) :
	Environment(InEnvironment), ModelData(InModelData)
{
}

TSharedPtr<NNE::IModelInstanceRDG> FModelORTDmlRDG::CreateModelInstanceRDG()
{
	const FRuntimeConf RuntimeConfig = Detail::MakeRuntimeConfigFromSettings(GetDefault<UNNERuntimeORTSettings>());

	TSharedPtr<FModelInstanceORTDmlRDG> ModelInstance = MakeShared<FModelInstanceORTDmlRDG>(ModelData, RuntimeConfig, Environment);
	if (!ModelInstance->Init())
	{
		return {};
	}

	return ModelInstance;
}

class FModelInstanceORTDmlRDG::FProxy
{
public:
	FProxy(TSharedRef<UE::NNE::FSharedModelData> InModelData, const FRuntimeConf& InRuntimeConf, TSharedRef<FEnvironment> InEnvironment)
		: ModelData(InModelData), RuntimeConf(InRuntimeConf), Environment(InEnvironment)
	{

	}
	
	~FProxy()
	{
		Session.Reset();
		if (!TempDirForModelWithExternalData.IsEmpty())
	{
		if (!IFileManager::Get().DeleteDirectory(*TempDirForModelWithExternalData, false, true))
		{
			UE_LOGF(LogNNERuntimeORT, Warning, "Large models are an experimental feature at the moment. FModelInstanceORTDmlRDG could not delete temp directy %ls on model instance destruction.", *TempDirForModelWithExternalData);
		}
	}
	}

	bool ConfigureTensors(bool bAreTensorInputs)
	{
		SCOPED_NAMED_EVENT_TEXT("FModelInstanceORTDmlRDG::ConfigureTensors", FColor::Magenta);

		const uint32 NumberTensors							= bAreTensorInputs ? Session->GetInputCount()			: Session->GetOutputCount();
		TArray<NNE::FTensorDesc>& SymbolicTensorDescs		= bAreTensorInputs ? InputSymbolicTensors		: OutputSymbolicTensors;
		TArray<ONNXTensorElementDataType>& TensorsORTType	= bAreTensorInputs ? InputTensorsORTType			: OutputTensorsORTType;
		TArray<char*>& TensorNames							= bAreTensorInputs ? InputTensorNames			: OutputTensorNames;
		TArray<Ort::AllocatedStringPtr>& TensorNameValues	= bAreTensorInputs ? InputTensorNameValues		: OutputTensorNameValues;
		TArray<TArray<FString>>& SymbolicDimensionNames 	= bAreTensorInputs ? InputSymbolicDimensionNames : OutputSymbolicDimensionNames;

		SymbolicTensorDescs.Reset();
		TensorsORTType.Reset();
		TensorNameValues.Reset();
		TensorNames.Reset();
		SymbolicDimensionNames.SetNum(NumberTensors);

		for (uint32 TensorIndex = 0; TensorIndex < NumberTensors; ++TensorIndex)
		{
			// Get Tensor name
			Ort::AllocatedStringPtr CurTensorName = bAreTensorInputs ? Session->GetInputNameAllocated(TensorIndex, *Allocator) : Session->GetOutputNameAllocated(TensorIndex, *Allocator);
			TensorNameValues.Emplace(MoveTemp(CurTensorName));
			TensorNames.Emplace(TensorNameValues.Last().get());

			// Get node type
			const Ort::TypeInfo CurrentTypeInfo = bAreTensorInputs ? Session->GetInputTypeInfo(TensorIndex) : Session->GetOutputTypeInfo(TensorIndex);
			const Ort::ConstTensorTypeAndShapeInfo CurrentTensorInfo = CurrentTypeInfo.GetTensorTypeAndShapeInfo();
			const ONNXTensorElementDataType ONNXTensorElementDataTypeEnum = CurrentTensorInfo.GetElementType();
			const TypeInfoORT TypeInfo = TranslateTensorTypeORTToNNE(ONNXTensorElementDataTypeEnum);

			// Get dynamic shape dimension names
			TUniquePtr<const char*[]> SymbolidDimensionNames = MakeUnique<const char*[]>(CurrentTensorInfo.GetShape().size());
			CurrentTensorInfo.GetSymbolicDimensions(SymbolidDimensionNames.Get(), CurrentTensorInfo.GetShape().size());

			TArray<FString>& CurrentSymbolicDimensionNames = SymbolicDimensionNames[TensorIndex];
			CurrentSymbolicDimensionNames.SetNum(CurrentTensorInfo.GetShape().size());

			for (int32 i = 0; i < CurrentTensorInfo.GetShape().size(); i++)
			{
				CurrentSymbolicDimensionNames[i] = FString((SymbolidDimensionNames.Get())[i]);
			}

			TensorsORTType.Emplace(ONNXTensorElementDataTypeEnum);

			// Get shape
			TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> ShapeData;
			ShapeData.Reserve(CurrentTensorInfo.GetShape().size());
			for (int64 CurrentTensorSize : CurrentTensorInfo.GetShape())
			{
				ShapeData.Add((int32)CurrentTensorSize);
			}

			const NNE::FSymbolicTensorShape Shape = NNE::FSymbolicTensorShape::Make(ShapeData);
			const NNE::FTensorDesc SymbolicTensorDesc = NNE::FTensorDesc::Make(FString(TensorNames.Last()), Shape, TypeInfo.DataType);

			check(SymbolicTensorDesc.GetElementByteSize() == TypeInfo.ElementSize);
			SymbolicTensorDescs.Emplace(SymbolicTensorDesc);
		}

		return true;
	}

	void SetOutputTensorShapes(TConstArrayView<NNE::FTensorShape> Shapes)
	{
		FScopeLock Lock(&CriticalSection);

		OutputTensorShapes = Shapes;
	}

	TConstArrayView<NNE::FTensorShape> GetOutputTensorShapes() const
	{
		FScopeLock Lock(&CriticalSection);

		return OutputTensorShapes;
	}

	TSharedRef<UE::NNE::FSharedModelData> ModelData;
	FRuntimeConf RuntimeConf;
	FString TempDirForModelWithExternalData;

	/** ORT-related variables */
	TSharedRef<FEnvironment> Environment;
	TUniquePtr<Ort::Session> Session;
	TUniquePtr<Ort::SessionOptions> SessionOptions;
	TUniquePtr<Ort::AllocatorWithDefaultOptions> Allocator;

	TArray<NNE::FTensorDesc>	InputSymbolicTensors;	// Constant
	TArray<NNE::FTensorDesc>	OutputSymbolicTensors;	// Should be constant, written/read by RHI Thread

	/** IO ORT-related variables */
	TArray<ONNXTensorElementDataType> InputTensorsORTType;
	TArray<ONNXTensorElementDataType> OutputTensorsORTType;

	TArray<Ort::AllocatedStringPtr> InputTensorNameValues;
	TArray<Ort::AllocatedStringPtr> OutputTensorNameValues;
	TArray<char*> InputTensorNames;
	TArray<char*> OutputTensorNames;
	TArray<TArray<FString>> InputSymbolicDimensionNames;
	TArray<TArray<FString>> OutputSymbolicDimensionNames;

	TArray<FTensor> OutputTensors;

private:
	mutable FCriticalSection CriticalSection;
	TArray<NNE::FTensorShape>	OutputTensorShapes;		// Written on RHI Thread, read on Game/Render Thread -> Sync
};

FModelInstanceORTDmlRDG::FModelInstanceORTDmlRDG(TSharedRef<UE::NNE::FSharedModelData> InModelData, const FRuntimeConf& InRuntimeConf, TSharedRef<FEnvironment> InEnvironment)
{
	Proxy = MakeShared<FProxy>(InModelData, InRuntimeConf, InEnvironment);
}

FModelInstanceORTDmlRDG::~FModelInstanceORTDmlRDG()
{
	
}

bool FModelInstanceORTDmlRDG::Init()
{
#if WITH_EDITOR
	try
#endif // WITH_EDITOR
	{
		Proxy->Allocator = MakeUnique<Ort::AllocatorWithDefaultOptions>();

		Proxy->SessionOptions = CreateSessionOptionsForDirectML(Proxy->Environment);
		if (!Proxy->SessionOptions.IsValid())
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Failed to configure session options for DirectML Execution Provider.");
			return false;
		}
		
		Proxy->SessionOptions->SetGraphOptimizationLevel(GetGraphOptimizationLevelForDML(true));

		if (!Detail::CreateSession(Proxy->ModelData->GetView(), *Proxy->SessionOptions, *Proxy->Environment, Proxy->Session, Proxy->TempDirForModelWithExternalData))
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Session creation failed.");
			return false;
		}

		if (!Proxy->ConfigureTensors(true))
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Failed to configure Inputs tensors.");
			return false;
		}
		if (!Proxy->ConfigureTensors(false))
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Failed to configure Outputs tensors.");
			return false;
		}

		InitialInputSymbolicTensors = Proxy->InputSymbolicTensors;
		InitialOutputSymbolicTensors = Proxy->OutputSymbolicTensors;
	}
#if WITH_EDITOR
	catch (const Ort::Exception& Exception)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "%ls", UTF8_TO_TCHAR(Exception.what()));
		return false;
	}
	catch (...)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Unknown exception!");
		return false;
	}
#endif // WITH_EDITOR

	return true;
}

TConstArrayView<NNE::FTensorDesc> FModelInstanceORTDmlRDG::GetInputTensorDescs() const
{
	return InitialInputSymbolicTensors;
}

TConstArrayView<NNE::FTensorShape> FModelInstanceORTDmlRDG::GetInputTensorShapes() const
{
	return InputTensorShapes;
}

TConstArrayView<NNE::FTensorDesc> FModelInstanceORTDmlRDG::GetOutputTensorDescs() const
{
	return InitialOutputSymbolicTensors;
}

TConstArrayView<NNE::FTensorShape> FModelInstanceORTDmlRDG::GetOutputTensorShapes() const
{
	return Proxy->GetOutputTensorShapes();
}


FModelInstanceORTDmlRDG::ESetInputTensorShapesStatus FModelInstanceORTDmlRDG::SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes)
{
	SCOPED_NAMED_EVENT_TEXT("FModelInstanceORTDmlRDG::SetInputTensorShapes", FColor::Magenta);

	if (IsInRenderingThread())
	{
		return SetInputTensorShapes_RenderThread(InInputShapes);
	}
	else
	{
		check(IsInGameThread());

		FEvent*	Signal = FGenericPlatformProcess::GetSynchEventFromPool();
		
		ENQUEUE_RENDER_COMMAND(FModelInstanceORTDmlRDG_SetInputTensorShapes)([this, InInputShapes, Signal](FRHICommandListImmediate& RHICmdList)
		{
			SetInputTensorShapes_RenderThread(InInputShapes);

			// note: Block here, if SetInputTensorShapes does actual GPU work!
			Signal->Trigger();
		});

		Signal->Wait();

		FGenericPlatformProcess::ReturnSynchEventToPool(Signal);
	}

	return ESetInputTensorShapesStatus::Ok;
}

// note: We delay shape inference to same RHI thread that runs the ORT session. This way we keep order of execution, but multiple calls to
// SetInputTensorShapes() followed by EnqueueRDG() do not interfere each other.
// SetInputTensorShapes() and EnqueueRDG() need to be called from the same thread, otherwise behaviour is undefined.
// SetInputTensorShapes() sets following members immediately, since they are accessible from the outside:
// - InputTensorShapes
// It also sets InputTensors, since its used by EnqueueRDG() on the same thread, but then needs to be preserved.
// OutputTensorShapes access requires synchronization, since its set by RHI thread and might be read by game/render thread
FModelInstanceORTDmlRDG::ESetInputTensorShapesStatus FModelInstanceORTDmlRDG::SetInputTensorShapes_RenderThread(TConstArrayView<NNE::FTensorShape> InInputShapes)
{
	SCOPED_NAMED_EVENT_TEXT("FModelInstanceORTDmlRDG::SetInputTensorShapes_RenderThread", FColor::Magenta);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	Proxy->SetOutputTensorShapes({});

	// Verify input shape are valid for the model and set InputTensorShapes
	if (!Detail::SetInputTensorShapes(InInputShapes, InitialInputSymbolicTensors, InputTensorShapes))
	{
		return ESetInputTensorShapesStatus::Fail;
	}

	// Set concrete input shapes
	InputTensors.Reset();
	for (int32 i = 0; i < InitialInputSymbolicTensors.Num(); i++)
	{
		FTensor Tensor = FTensor::Make(InputTensorShapes[i], InitialInputSymbolicTensors[i].GetDataType());
		InputTensors.Emplace(Tensor);
	}

	RHICmdList.EnqueueLambda([Proxy = Proxy, InputTensorShapes = InputTensorShapes](FRHICommandListImmediate& RHICmdList)
	{
		GetID3D12PlatformDynamicRHI()->RHIRunOnQueue(ED3D12RHIRunOnQueueType::Graphics, [Proxy = Proxy, InputTensorShapes = InputTensorShapes] (ID3D12CommandQueue* D3D12CommandQueue)
		{
			SCOPED_NAMED_EVENT_TEXT("FModelInstanceORTDmlRDG::SetInputTensorShapes RHIRunOnQueue", FColor::Magenta);

			Proxy->OutputTensors.Reset();
			TArray<NNE::FTensorShape>	ResultOutputTensorShapes;

			// Check whether all input tensor shapes are concrete
			bool bHasSymbolicInputShapes = false;
			for (int32 i = 0; i < Proxy->InputSymbolicTensors.Num(); i++)
			{
				if (!Proxy->InputSymbolicTensors[i].GetShape().IsConcrete())
				{
					bHasSymbolicInputShapes = true;

					break;
				}
			}

			if (!bHasSymbolicInputShapes)
			{
				// All output shapes need to be concrete now
				for (int32 i = 0; i < Proxy->OutputSymbolicTensors.Num(); i++)
				{
					const NNE::FTensorDesc SymbolicTensorDesc = Proxy->OutputSymbolicTensors[i];

					if (SymbolicTensorDesc.GetShape().IsConcrete())
					{
						FTensor Tensor = FTensor::MakeFromSymbolicDesc(SymbolicTensorDesc);
						Proxy->OutputTensors.Emplace(Tensor);
						ResultOutputTensorShapes.Emplace(Tensor.GetShape());
					}
					else
					{
						UE_LOGF(LogNNERuntimeORT, Warning, "One or more output tensors contain free dimensions, but input tensors are all concrete!");
						return;
					}
				}

				Proxy->SetOutputTensorShapes(ResultOutputTensorShapes);

				return;
			}

			// Recreate session options because potentially we add new free dimension overrides
			Proxy->SessionOptions = CreateSessionOptionsForDirectML(Proxy->Environment);
			if (!Proxy->SessionOptions.IsValid())
			{
				UE_LOGF(LogNNERuntimeORT, Error, "Failed to recreate session options!");
				return;
			}

			Proxy->SessionOptions->SetGraphOptimizationLevel(GetGraphOptimizationLevelForDML(true));

			// Setup concrete input tensors
			for (int32 i = 0; i < Proxy->InputSymbolicTensors.Num(); i++)
			{
				const NNE::FSymbolicTensorShape& SymbolicInputShape = Proxy->InputSymbolicTensors[i].GetShape();

				// Override free dimensions of input tensors
				if (!SymbolicInputShape.IsConcrete())
				{
					check(InputTensorShapes[i].IsCompatibleWith(SymbolicInputShape));

					TConstArrayView<int32> InputSymbolicShapeData = SymbolicInputShape.GetData();
					TConstArrayView<uint32> InputShapeData = InputTensorShapes[i].GetData();

					for (int32 j = 0; j < InputShapeData.Num(); j++)
					{
						if (InputSymbolicShapeData[j] < 0)
						{
							Ort::GetApi().AddFreeDimensionOverrideByName(*Proxy->SessionOptions, TCHAR_TO_ANSI(*Proxy->InputSymbolicDimensionNames[i][j]), InputShapeData[j]);
						}
					}
				}
			}

			if (!Detail::CreateSession(Proxy->ModelData->GetView(), *Proxy->SessionOptions, *Proxy->Environment, Proxy->Session, Proxy->TempDirForModelWithExternalData))
			{
				UE_LOGF(LogNNERuntimeORT, Error, "Failed to recreate session!");
				return;
			}

			// Need to configure output tensors with new session (to apply free dimension overrides)
			if (!Proxy->ConfigureTensors(false))
			{
				UE_LOGF(LogNNERuntimeORT, Error, "Failed to configure tensors!");
				return;
			}

			// All output shapes need to be concrete now
			for (int32 i = 0; i < Proxy->OutputSymbolicTensors.Num(); i++)
			{
				const NNE::FTensorDesc SymbolicTensorDesc = Proxy->OutputSymbolicTensors[i];

				if (SymbolicTensorDesc.GetShape().IsConcrete())
				{
					FTensor Tensor = FTensor::MakeFromSymbolicDesc(SymbolicTensorDesc);
					Proxy->OutputTensors.Emplace(Tensor);
					ResultOutputTensorShapes.Emplace(Tensor.GetShape());
				}
				else
				{
					for (int32 j = 0; j < SymbolicTensorDesc.GetShape().Rank(); j++)
					{
						if (SymbolicTensorDesc.GetShape().GetData()[j] < 0)
						{
							UE_LOGF(LogNNERuntimeORT, Warning, "Tensor '%s' has free dimension '%ls'.", Proxy->OutputTensorNames[i], *Proxy->OutputSymbolicDimensionNames[i][j]);
						}
					}

					UE_LOGF(LogNNERuntimeORT, Error, "One or more output tensors contain free dimensions!");
					return;
				}
			}

			Proxy->SetOutputTensorShapes(ResultOutputTensorShapes);
		}, false);
	});

	return ESetInputTensorShapesStatus::Ok;
}

Ort::Value CreateTensor(const OrtDmlApi& DmlApi, const Ort::MemoryInfo& MemoryInfo, FRHIBuffer* Buffer, const FTensor& Tensor, ONNXTensorElementDataType ElementDataType,
	TArray<std::unique_ptr<void, void (*)(void*)>>& DmlAllocatorResources)
{
	checkf(Buffer, TEXT("CreateTensor needs Buffer to be set"));

	ID3D12Resource* NativeD3D12Resource = GetID3D12DynamicRHI()->RHIGetResource(Buffer);

	void* DmlAllocatorResourcePtr;
	Ort::ThrowOnError(DmlApi.CreateGPUAllocationFromD3DResource(NativeD3D12Resource, &DmlAllocatorResourcePtr));

	std::unique_ptr<void, void (*)(void*)> DmlAllocatorResource(DmlAllocatorResourcePtr,
		[] (void* Ptr)
	{
		const OrtDmlApi* DmlApi;
		Ort::ThrowOnError(Ort::GetApi().GetExecutionProviderApi("DML", ORT_API_VERSION, reinterpret_cast<const void**>(&DmlApi)));

		DmlApi->FreeGPUAllocation(Ptr);
	});

	uint64 SizeInBytes = static_cast<uint64>(NativeD3D12Resource->GetDesc().Width);

	TUniquePtr<int64_t[]> Shape = MakeUnique<int64_t[]>(Tensor.GetShape().Rank());
	for (int32 i = 0; i < Tensor.GetShape().Rank(); ++i)
	{
		Shape.Get()[i] = Tensor.GetShape().GetData()[i];
	}
	const uint32 ShapeLen{ (uint32)Tensor.GetShape().Rank() };

	Ort::Value Result = Ort::Value::CreateTensor(MemoryInfo, DmlAllocatorResource.get(), SizeInBytes, Shape.Get(), ShapeLen, ElementDataType);

	DmlAllocatorResources.Add(MoveTemp(DmlAllocatorResource));

	return Result;
}

// note: make sure to copy anything read by the RHI thread, but set by the calling thread. Currently only member InputTensors is affected.
// Everything else is constant (set on Init()) or set in the same RHI thread in the lambda enqueued by SetInputTensorShapes().
// As of today and without any additional settings/flags, the lambda passed to RHICmdList.EnqueueLambda([](){}) is executed on the RHI thread and
// the lambda passed to GetID3D12PlatformDynamicRHI()->RHIRunOnQueue(..., [](){}, ...) is executed on the RHI submission thread.
FModelInstanceORTDmlRDG::EEnqueueRDGStatus FModelInstanceORTDmlRDG::EnqueueRDG(FRDGBuilder& GraphBuilder, TConstArrayView<NNE::FTensorBindingRDG> Inputs, TConstArrayView<NNE::FTensorBindingRDG> Outputs)
{
	SCOPED_NAMED_EVENT_TEXT("FModelInstanceORTDmlRDG::EnqueueRDG", FColor::Magenta);

	if (InputTensorShapes.IsEmpty())
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Input shapes are not set, please call SetInputTensorShapes.");
		return EEnqueueRDGStatus::Fail;
	}

	if (Inputs.Num() != InitialInputSymbolicTensors.Num())
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Input bindings need to match input tensor descriptor count (got %d, expected %d).", Inputs.Num(), InitialInputSymbolicTensors.Num());
		return EEnqueueRDGStatus::Fail;
	}

	if (!Outputs.IsEmpty() && Outputs.Num() != InitialOutputSymbolicTensors.Num())
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Output binding can be empty or needs to match output tensor descriptor count (got %d, expected %d).", Outputs.Num(), InitialOutputSymbolicTensors.Num());
		return EEnqueueRDGStatus::Fail;
	}

	FORTModelInstanceRDGParameters* PassParameters = GraphBuilder.AllocParameters<FORTModelInstanceRDGParameters>();
	for (int32 i = 0; i < Inputs.Num(); i++)
	{
		const NNE::FTensorBindingRDG& Binding = Inputs[i];
		if (!Binding.Buffer && InputTensors[i].GetDataSize() != 0)
		{
			UE_LOG(LogNNERuntimeORT, Error, TEXT("Binding input tensor %d is not set but given size by tensor descriptor is non-zero %" UINT64_FMT "."), i, InputTensors[i].GetDataSize());
			return EEnqueueRDGStatus::Fail;
		}

		PassParameters->InputBuffers.Emplace(Binding.Buffer, ERHIAccess::CopySrc);
	}

	for (int32 i = 0; i < Outputs.Num(); i++)
	{
		PassParameters->OutputBuffers.Emplace(Outputs[i].Buffer, ERHIAccess::CopyDest);
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNERuntimeORTDmlRDG, "FModelInstanceORTDmlRDG::EnqueueRDG");

	GraphBuilder.AddPass(RDG_EVENT_NAME("FModelInstanceORTDmlRDG::EnqueueRDG.AddPass"), PassParameters, ERDGPassFlags::Readback,
	[Proxy = Proxy, InputTensors = InputTensors, PassParameters](FRHICommandListImmediate& RHICmdList)
	{
		SCOPED_NAMED_EVENT_TEXT("FModelInstanceORTDmlRDG::EnqueueRDG.AddPass", FColor::Magenta);

		TArray<FRHIBuffer*> InputBuffers;
		InputBuffers.SetNumUninitialized(PassParameters->InputBuffers.Num());
		for (int32 i = 0; i < PassParameters->InputBuffers.Num(); i++)
		{
			InputBuffers[i] = PassParameters->InputBuffers[i]->GetRHI();
		}

		TArray<FRHIBuffer*> OutputBuffers;
		OutputBuffers.SetNumZeroed(Proxy->OutputSymbolicTensors.Num());
		for (int32 i = 0; i < PassParameters->OutputBuffers.Num(); i++)
		{
			OutputBuffers[i] = PassParameters->OutputBuffers[i]->GetRHI();
		}

		// Submit previous work here to the GPU to avoid ORT Session Run() dispatching its work first
		RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

		RHICmdList.EnqueueLambda([Proxy = Proxy, InputTensors = InputTensors, InputBuffers = InputBuffers, OutputBuffers = OutputBuffers] (FRHICommandListImmediate& RHICmdList)
		{
			GetID3D12PlatformDynamicRHI()->RHIRunOnQueue(ED3D12RHIRunOnQueueType::Graphics, [Proxy = Proxy, InputTensors = InputTensors, InputBuffers = InputBuffers, OutputBuffers = OutputBuffers] (ID3D12CommandQueue* D3D12CommandQueue)
			{
				SCOPED_NAMED_EVENT_TEXT("FModelInstanceORTDmlRDG::EnqueueRDG.AddPass RHIRunOnQueue", FColor::Magenta);
#if WITH_EDITOR
				try
#endif // WITH_EDITOR
				{
					if (!Proxy->Session.IsValid())
					{
						UE_LOGF(LogNNERuntimeORT, Error, "Invalid Session, may be Init() should have been called.");
						return;
					}

					const OrtDmlApi* DmlApi;
					Ort::ThrowOnError(Ort::GetApi().GetExecutionProviderApi("DML", ORT_API_VERSION, reinterpret_cast<const void**>(&DmlApi)));
					
					Ort::MemoryInfo MemoryInfo("DML", OrtAllocatorType::OrtDeviceAllocator, 0, OrtMemType::OrtMemTypeDefault);
					
					TArray<std::unique_ptr<void, void (*)(void*)>> DmlAllocatorResources;
					TArray<Ort::Value> OrtInputTensors;
					TArray<Ort::Value> OrtOutputTensors;

					for (int32 i = 0; i < InputBuffers.Num(); i++)
					{
						const uint64 DmlImpliedSizeBytes = CalcRDGBufferSizeForDirectML(InputTensors[i].GetDataSize());
						if (InputBuffers[i] && InputBuffers[i]->GetDesc().Size < DmlImpliedSizeBytes)
						{
							UE_LOG(LogNNERuntimeORT, Error, TEXT("Binding input tensor %d size does not match tensor buffer size required by DirectML (got %d, expected %" UINT64_FMT ", data size was %" UINT64_FMT ")."), i, InputBuffers[i]->GetDesc().Size, DmlImpliedSizeBytes, InputTensors[i].GetDataSize());
							return;
						}
						
						OrtInputTensors.Add(CreateTensor(*DmlApi, MemoryInfo, InputBuffers[i], InputTensors[i], Proxy->InputTensorsORTType[i], DmlAllocatorResources));
					}
					for (int32 i = 0; i < OutputBuffers.Num(); i++)
					{
						const uint64 DmlImpliedSizeBytes = CalcRDGBufferSizeForDirectML(Proxy->OutputTensors[i].GetDataSize());
						if (OutputBuffers[i] && OutputBuffers[i]->GetDesc().Size < DmlImpliedSizeBytes)
						{
							UE_LOG(LogNNERuntimeORT, Error, TEXT("Binding output tensor %d size does not match tensor buffer size required by DirectML (got %d, expected %" UINT64_FMT ", data size was %" UINT64_FMT ")."), i, OutputBuffers[i]->GetDesc().Size, DmlImpliedSizeBytes, Proxy->OutputTensors[i].GetDataSize());
							return;
						}

						if (OutputBuffers[i])
						{
							OrtOutputTensors.Add(CreateTensor(*DmlApi, MemoryInfo, OutputBuffers[i], Proxy->OutputTensors[i], Proxy->OutputTensorsORTType[i], DmlAllocatorResources));
						}
						else
						{
							OrtOutputTensors.Emplace(nullptr);
						}
					}

					Proxy->Session->Run(Ort::RunOptions{ nullptr },
						Proxy->InputTensorNames.GetData(), &OrtInputTensors[0], Proxy->InputTensorNames.Num(),
						Proxy->OutputTensorNames.GetData(), &OrtOutputTensors[0], Proxy->OutputTensorNames.Num());
				}
#if WITH_EDITOR
				catch (const Ort::Exception& Exception)
				{
					UE_LOGF(LogNNERuntimeORT, Error, "ORT Exception: %ls", UTF8_TO_TCHAR(Exception.what()));
				}
				catch (...)
				{
					UE_LOGF(LogNNERuntimeORT, Error, "ORT Exception: Unknown!");
				}
#endif // WITH_EDITOR
			}, false);
		});
	});

	return EEnqueueRDGStatus::Ok;
}
#endif //PLATFORM_WINDOWS
	
} // namespace UE::NNERuntimeORT::Private