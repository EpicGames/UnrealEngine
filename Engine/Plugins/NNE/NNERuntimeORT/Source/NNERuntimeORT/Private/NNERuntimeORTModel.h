// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNEModelData.h"
#include "NNEOnnxruntime.h"
#include "NNERuntimeCPU.h"
#include "NNERuntimeGPU.h"
#include "NNERuntimeORTEnv.h"
#include "NNERuntimeORTTensor.h"
#include "NNERuntimeRDG.h"
#include "NNETypes.h"

namespace UE::NNERuntimeORT::Private
{

struct FRuntimeConf
{
	ExecutionMode ExecutionMode = ExecutionMode::ORT_SEQUENTIAL;
};

template <class ModelInterface>
class FModelInstanceORTBase : public ModelInterface
{
public:

	virtual ~FModelInstanceORTBase() = default;

	virtual TConstArrayView<NNE::FTensorDesc> GetInputTensorDescs() const override;
	virtual TConstArrayView<NNE::FTensorDesc> GetOutputTensorDescs() const override;
	virtual TConstArrayView<NNE::FTensorShape> GetInputTensorShapes() const override;
	virtual TConstArrayView<NNE::FTensorShape> GetOutputTensorShapes() const override;
	virtual typename ModelInterface::ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes) override;

protected:

	FModelInstanceORTBase() {}

	TArray<NNE::FTensorShape>	InputTensorShapes;
	TArray<NNE::FTensorShape>	OutputTensorShapes;
	TArray<NNE::FTensorDesc>	InputSymbolicTensors;
	TArray<NNE::FTensorDesc>	OutputSymbolicTensors;
};

template <class ModelInterface, class TensorBinding>
class FModelInstanceORTRunSync : public FModelInstanceORTBase<ModelInterface>
{

public:
	FModelInstanceORTRunSync(const FRuntimeConf& InRuntimeConf, TSharedRef<FEnvironment> InEnvironment);
	virtual ~FModelInstanceORTRunSync();

	virtual typename ModelInterface::ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes) override;

	bool Init(TConstArrayView64<uint8> ModelData);

	typename ModelInterface::ERunSyncStatus RunSync(TConstArrayView<TensorBinding> InInputBindings, TConstArrayView<TensorBinding> InOutputBindings) override;

protected:
	virtual bool InitializedAndConfigureMembers();
	bool ConfigureTensors(const bool InIsInput);

	FRuntimeConf RuntimeConf;
	FString TempDirForModelWithExternalData;

	/** ORT-related variables */
	TSharedRef<FEnvironment> Environment;
	TUniquePtr<Ort::Session> Session;
	TUniquePtr<Ort::AllocatorWithDefaultOptions> Allocator;
	TUniquePtr<Ort::SessionOptions> SessionOptions;
	TUniquePtr<Ort::MemoryInfo> MemoryInfo;

	/** IO ORT-related variables */
	TArray<ONNXTensorElementDataType> InputTensorsORTType;
	TArray<ONNXTensorElementDataType> OutputTensorsORTType;

	TArray<Ort::AllocatedStringPtr> InputTensorNameValues;
	TArray<Ort::AllocatedStringPtr> OutputTensorNameValues;
	TArray<char*> InputTensorNames;
	TArray<char*> OutputTensorNames;

	TArray<FTensor> InputTensors;
	TArray<FTensor> OutputTensors;
};

class FModelInstanceORTCpu : public FModelInstanceORTRunSync<NNE::IModelInstanceCPU, NNE::FTensorBindingCPU>
{
public:
	FModelInstanceORTCpu(const FRuntimeConf& InRuntimeConf, TSharedRef<FEnvironment> InEnvironment) : FModelInstanceORTRunSync(InRuntimeConf, InEnvironment) {}
	virtual ~FModelInstanceORTCpu() = default;

private:
	virtual bool InitializedAndConfigureMembers() override;
};

class FModelORTCpu : public NNE::IModelCPU
{
public:
	FModelORTCpu(TSharedRef<FEnvironment> InEnvironment, TSharedRef<UE::NNE::FSharedModelData> InModelData);
	virtual ~FModelORTCpu() = default;

	virtual TSharedPtr<NNE::IModelInstanceCPU> CreateModelInstanceCPU() override;

private:
	TSharedRef<FEnvironment> Environment;
	TSharedRef<UE::NNE::FSharedModelData> ModelData;
};

#if PLATFORM_WINDOWS
class FModelORTDmlGPU : public NNE::IModelGPU
{
public:
	FModelORTDmlGPU(TSharedRef<FEnvironment> InEnvironment, TSharedRef<UE::NNE::FSharedModelData> InModelData);
	virtual ~FModelORTDmlGPU() = default;

	virtual TSharedPtr<NNE::IModelInstanceGPU> CreateModelInstanceGPU() override;

private:
	TSharedRef<FEnvironment> Environment;
	TSharedRef<UE::NNE::FSharedModelData> ModelData;
};

class FModelInstanceORTDmlGPU : public FModelInstanceORTRunSync<NNE::IModelInstanceGPU, NNE::FTensorBindingCPU>
{
public:
	FModelInstanceORTDmlGPU(const FRuntimeConf& InRuntimeConf, TSharedRef<FEnvironment> InEnvironment) : FModelInstanceORTRunSync(InRuntimeConf, InEnvironment) {}
	virtual ~FModelInstanceORTDmlGPU() = default;

private:
	virtual bool InitializedAndConfigureMembers() override;
};

class FModelORTDmlRDG : public NNE::IModelRDG
{
public:
	FModelORTDmlRDG(TSharedRef<FEnvironment> InEnvironment, TSharedRef<UE::NNE::FSharedModelData> InModelData);

	virtual ~FModelORTDmlRDG() = default;

	virtual TSharedPtr<NNE::IModelInstanceRDG> CreateModelInstanceRDG() override;

private:
	TSharedRef<FEnvironment> Environment;
	TSharedRef<UE::NNE::FSharedModelData> ModelData;
};

class FModelInstanceORTDmlRDG : public NNE::IModelInstanceRDG
{

public:
	using ESetInputTensorShapesStatus = NNE::IModelInstanceRDG::ESetInputTensorShapesStatus;
	using EEnqueueRDGStatus = NNE::IModelInstanceRDG::EEnqueueRDGStatus;

	FModelInstanceORTDmlRDG(TSharedRef<UE::NNE::FSharedModelData> InModelData, const FRuntimeConf& InRuntimeConf, TSharedRef<FEnvironment> InEnvironment);

	virtual ~FModelInstanceORTDmlRDG();

	TConstArrayView<NNE::FTensorDesc> GetInputTensorDescs() const override;
	TConstArrayView<NNE::FTensorDesc> GetOutputTensorDescs() const override;
	TConstArrayView<NNE::FTensorShape> GetInputTensorShapes() const override;
	TConstArrayView<NNE::FTensorShape> GetOutputTensorShapes() const override;

	ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes) override;
	ESetInputTensorShapesStatus SetInputTensorShapes_RenderThread(TConstArrayView<NNE::FTensorShape> InInputShapes);

	bool Init();

	EEnqueueRDGStatus EnqueueRDG(FRDGBuilder& GraphBuilder, TConstArrayView<NNE::FTensorBindingRDG> Inputs, TConstArrayView<NNE::FTensorBindingRDG> Outputs) override;

private:
	TArray<NNE::FTensorShape>	InputTensorShapes; // Owned by Game/Render thread, copied for RHI thread
	TArray<FTensor> InputTensors; // Owned by Game/Render thread, copied for RHI thread

	TArray<NNE::FTensorDesc> InitialInputSymbolicTensors; // Copy for Game/Render Thread
	TArray<NNE::FTensorDesc> InitialOutputSymbolicTensors; // Copy for Game/Render Thread

	class FProxy; // Contains data written/read by RHI Thread and data initialized by Game/Render Thread
	TSharedPtr<FProxy> Proxy;
};
#endif //PLATFORM_WINDOWS
	
} // UE::NNERuntimeORT::Private