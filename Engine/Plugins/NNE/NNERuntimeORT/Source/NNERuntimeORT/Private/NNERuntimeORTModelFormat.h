// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/Archive.h"

namespace UE::NNERuntimeORT::Private
{
	struct FOnnxAdditionalDataDescriptor
	{
		FString Path;
		int64 Offset;
		int64 Size;
	};

	FORCEINLINE FArchive& operator <<(FArchive& Ar, FOnnxAdditionalDataDescriptor& AdditionalDataDescriptor)
	{
		Ar << AdditionalDataDescriptor.Path;
		Ar << AdditionalDataDescriptor.Offset;
		Ar << AdditionalDataDescriptor.Size;
		return Ar;
	}

	struct FOnnxDataDescriptor
	{

		int64 OnnxModelDataSize = 0;
		TArray<FOnnxAdditionalDataDescriptor> AdditionalDataDescriptors;
	};

	FORCEINLINE FArchive& operator <<(FArchive& Ar, FOnnxDataDescriptor& OnnxDataDescriptor)
	{
		Ar << OnnxDataDescriptor.OnnxModelDataSize;
		Ar << OnnxDataDescriptor.AdditionalDataDescriptors;
		return Ar;
	}

} // namespace UE::NNERuntimeORT::Private