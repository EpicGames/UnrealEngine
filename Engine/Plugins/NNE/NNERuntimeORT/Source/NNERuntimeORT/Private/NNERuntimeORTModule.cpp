// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeORTModule.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "NNE.h"
#include "NNEOnnxruntime.h"
#include "NNERuntimeORT.h"
#include "NNERuntimeORTEnv.h"
#include "NNERuntimeORTSettings.h"
#include "UObject/WeakInterfacePtr.h"

DEFINE_LOG_CATEGORY(LogNNEOnnxruntime);

namespace UE::NNERuntimeORT::Private
{
	namespace DllHelper
	{
		bool GetDllHandle(const FString& DllPath, TArray<void*>& DllHandles)
		{
			void *DllHandle = nullptr;

			if (!FPaths::FileExists(DllPath))
			{
				UE_LOGF(LogNNERuntimeORT, Error, "Failed to find the third party library %ls.", *DllPath);
				return false;
			}
			
			DllHandle = FPlatformProcess::GetDllHandle(*DllPath);

			if (!DllHandle)
			{
				UE_LOGF(LogNNERuntimeORT, Error, "Failed to load the third party library %ls.", *DllPath);
				return false;
			}

			DllHandles.Add(DllHandle);
			return true;
		}
	} // namespace DllHelper

	namespace EnvironmentHelper
	{
		void CreateOrtEnvFromSettings(const UNNERuntimeORTSettings* Settings, FEnvironment& Environment)
		{
#if WITH_EDITOR
			FThreadingOptions ThreadingOptions = Settings->EditorThreadingOptions;
#else
			FThreadingOptions ThreadingOptions = Settings->GameThreadingOptions;
#endif

			UE::NNERuntimeORT::Private::FEnvironment::FConfig Config{};
			Config.bUseGlobalThreadPool = ThreadingOptions.bUseGlobalThreadPool;
			Config.IntraOpNumThreads = ThreadingOptions.IntraOpNumThreads;
			Config.InterOpNumThreads = ThreadingOptions.InterOpNumThreads;

			Environment.Configure(Config);
		}
	} // EnvironmentHelper
} // namespace UE::NNERuntimeRDG::Private

void FNNERuntimeORTModule::StartupModule()
{
	using namespace UE::NNERuntimeORT::Private;

	const FString PluginDir = IPluginManager::Get().FindPlugin("NNERuntimeORT")->GetBaseDir();
	const FString OrtSharedLibPath = FPaths::Combine(PluginDir, TEXT(UE_STRINGIZE(ONNXRUNTIME_SHAREDLIB_PATH)));

	if (!DllHelper::GetDllHandle(OrtSharedLibPath, DllHandles))
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Failed to load ONNX Runtime shared library. ORT Runtimes won't be available.");
		return;
	}

	void* OrtDllHandle = DllHandles.Last();

	bool bDirectMLDllLoaded = false;

#if PLATFORM_WINDOWS
	const FString ModuleDir = FPlatformProcess::GetModulesDirectory();
	const FString DirectMLSharedLibPath = FPaths::Combine(ModuleDir, TEXT(UE_STRINGIZE(DIRECTML_PATH)), TEXT("DirectML.dll"));

	bDirectMLDllLoaded = DllHelper::GetDllHandle(DirectMLSharedLibPath, DllHandles);
	if (!bDirectMLDllLoaded)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Failed to load DirectML shared library. ORT Dml Runtime won't be available.");
	}
#endif // PLATFORM_WINDOWS

	TUniquePtr<UE::NNEOnnxruntime::OrtApiFunctions> OrtApiFunctions = UE::NNEOnnxruntime::LoadApiFunctions(OrtDllHandle);
	if (!OrtApiFunctions.IsValid())
	{
		UE_LOGF(LogNNERuntimeORT, Fatal, "Failed to load ONNX Runtime shared library functions!");
		return;
	}

	Ort::InitApi(OrtApiFunctions->OrtGetApiBase()->GetApi(ORT_API_VERSION));
	
	Environment = MakeShared<FEnvironment>();

	EnvironmentHelper::CreateOrtEnvFromSettings(GetDefault<UNNERuntimeORTSettings>(), *Environment);

	// NNE runtime ORT Dml startup
	NNERuntimeORTDml = MakeRuntimeDml(bDirectMLDllLoaded);
	if (NNERuntimeORTDml.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeDmlInterface(NNERuntimeORTDml.Get());

		NNERuntimeORTDml->Init(Environment.ToSharedRef(), bDirectMLDllLoaded);
		NNERuntimeORTDml->AddToRoot();
		UE::NNE::RegisterRuntime(RuntimeDmlInterface);
	}

	// NNE runtime ORT Cpu startup
	NNERuntimeORTCpu = NewObject<UNNERuntimeORTCpu>();
	if (NNERuntimeORTCpu.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeCPUInterface(NNERuntimeORTCpu.Get());

		NNERuntimeORTCpu->Init(Environment.ToSharedRef());
		NNERuntimeORTCpu->AddToRoot();
		UE::NNE::RegisterRuntime(RuntimeCPUInterface);
	}

#if WITH_EDITOR
	GetMutableDefault<UNNERuntimeORTSettings>()->OnSettingChanged().AddRaw(this, &FNNERuntimeORTModule::OnSettingsChanged);
#endif
}

void FNNERuntimeORTModule::ShutdownModule()
{
#if WITH_EDITOR
	if(UObjectInitialized())
	{
		GetMutableDefault<UNNERuntimeORTSettings>()->OnSettingChanged().RemoveAll(this);
	}
#endif

	// NNE runtime ORT Cpu shutdown
	if (NNERuntimeORTCpu.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeCPUInterface(NNERuntimeORTCpu.Get());

		UE::NNE::UnregisterRuntime(RuntimeCPUInterface);
		NNERuntimeORTCpu->RemoveFromRoot();
		NNERuntimeORTCpu.Reset();
	}

	// NNE runtime ORT Dml shutdown
	if (NNERuntimeORTDml.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeDmlInterface(NNERuntimeORTDml.Get());

		UE::NNE::UnregisterRuntime(RuntimeDmlInterface);
		NNERuntimeORTDml->RemoveFromRoot();
		NNERuntimeORTDml.Reset();
	}

	Environment.Reset();

	// Free the dll handles
	for(void* DllHandle : DllHandles)
	{
		FPlatformProcess::FreeDllHandle(DllHandle);
	}
	DllHandles.Empty();
}

#if WITH_EDITOR
void FNNERuntimeORTModule::OnSettingsChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	UE_LOGF(LogNNERuntimeORT, Log, "Settings %ls changed: %ls", *InObject->GetName(), *InPropertyChangedEvent.GetPropertyName().ToString());
	UE_LOGF(LogNNERuntimeORT, Warning, "It is recommended to restart the Editor if settings %ls changed! Otherwise they might not be fully applied.", *InObject->GetName());

	UE::NNERuntimeORT::Private::EnvironmentHelper::CreateOrtEnvFromSettings(CastChecked<UNNERuntimeORTSettings>(InObject), *Environment);
}
#endif

IMPLEMENT_MODULE(FNNERuntimeORTModule, NNERuntimeORT);