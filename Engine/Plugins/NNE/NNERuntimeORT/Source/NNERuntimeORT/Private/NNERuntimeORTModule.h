// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Modules/ModuleManager.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UNNERuntimeORTDmlProxy;
class UNNERuntimeORTCpu;

namespace UE::NNERuntimeORT::Private
{
	class FEnvironment;
}

class FNNERuntimeORTModule : public IModuleInterface
{
private:
	TWeakObjectPtr<UNNERuntimeORTDmlProxy> NNERuntimeORTDml{ nullptr };
	TWeakObjectPtr<UNNERuntimeORTCpu> NNERuntimeORTCpu{ nullptr };

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

#if WITH_EDITOR
void OnSettingsChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent);
#endif

	TArray<void*> DllHandles;
	TSharedPtr<UE::NNERuntimeORT::Private::FEnvironment> Environment;
};