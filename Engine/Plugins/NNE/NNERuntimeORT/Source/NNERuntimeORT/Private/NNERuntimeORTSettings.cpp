// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeORTSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NNERuntimeORTSettings)

UNNERuntimeORTSettings::UNNERuntimeORTSettings(const FObjectInitializer& ObjectInitlaizer)
	: Super(ObjectInitlaizer)
{

}

FName UNNERuntimeORTSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UNNERuntimeORTSettings::GetSectionText() const
{
	return NSLOCTEXT("NNERuntimeORTPlugin", "NNERuntimeORTSettingsSection", "NNERuntimeORT");
}
#endif
