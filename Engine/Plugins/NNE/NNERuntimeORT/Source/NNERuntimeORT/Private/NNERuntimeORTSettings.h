// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "NNERuntimeORTSettings.generated.h"

/** An enum to represent denoiser NNE runtime type */
UENUM()
enum EExecutionMode : uint8
{
	SEQUENTIAL,
	PARALLEL
};

/** Threading options to configure glocal/local thread pools and exection mode */
USTRUCT()
struct FThreadingOptions
{
	GENERATED_BODY()

	/** Use global threadpools that will be shared across sessions */
	UPROPERTY(Config, EditAnywhere, Category = "ONNX Runtime", meta = (
		DisplayName = "Use global thread pool",
		ToolTip = "Use global threadpools that will be shared across sessions."))
	bool bUseGlobalThreadPool = false;

	/** Thread count to parallelize the execution within nodes */
	UPROPERTY(Config, EditAnywhere, Category = "ONNX Runtime", meta = (
		DisplayName = "Intra-op thread count",
		ToolTip = 	"Set thread count of intra-op thread pool, which is utilized by ONNX Runtime to parallelize computation inside each operator.\nSpecial values:\n   0 = Use default thread count\n   1 = The invoking thread will be used; no threads will be created in the thread pool."))
	int32 IntraOpNumThreads = 0;

	/** Thread count used to parallelize the execution of the graph */
	UPROPERTY(Config, EditAnywhere, Category = "ONNX Runtime", meta = (
		DisplayName = "Inter-op thread count",
		ToolTip = 	"Set thread count of the inter-op thread pool, which enables parallelism between operators and will only be created when session execution mode set to parallel.\nSpecial values:\n   0 = Use default thread count\n   1 = The invoking thread will be used; no threads will be created in the thread pool."))
	int32 InterOpNumThreads = 0;

	/** Execution mode controls whether multiple operators in the graph (across nodes) run sequentially or in parallel */
	UPROPERTY(Config, EditAnywhere, Category = "ONNX Runtime", meta = (
		DisplayName = "Execution mode",
		ToolTip = 	"Controls whether multiple operators in the graph (across nodes) run sequentially or in parallel.\nNote: DirectML EP requires sequential execution and therefore ignores this setting."))
	TEnumAsByte<EExecutionMode> ExecutionMode = EExecutionMode::SEQUENTIAL;
};

/** Settings used to configure NNERuntimeORT */
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "NNERuntimeORT"))
class UNNERuntimeORTSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	UNNERuntimeORTSettings(const FObjectInitializer& ObjectInitlaizer);

public:

	/** Threading options in Editor targets */
	UPROPERTY(Config, EditAnywhere, Category = "ONNX Runtime", meta = (DisplayName = "Editor threading options", ToolTip = "Threading options in Editor targets", ConfigRestartRequired = true))
	FThreadingOptions EditorThreadingOptions{true, 0, 0, EExecutionMode::SEQUENTIAL};

	/** Threading options in Non-Editor (Game, Program, ...) targets */
	UPROPERTY(Config, EditAnywhere, Category = "ONNX Runtime", meta = (DisplayName = "Game threading options", ToolTip = "Threading options in Non-Editor (Game, Program, ...) targets", ConfigRestartRequired = true))
	FThreadingOptions GameThreadingOptions{false, 1, 1, EExecutionMode::SEQUENTIAL};

	// Begin UDeveloperSettings Interface
	NNERUNTIMEORT_API virtual FName GetCategoryName() const override;
#if WITH_EDITOR
	NNERUNTIMEORT_API virtual FText GetSectionText() const override;
	// END UDeveloperSettings Interface
#endif

};
