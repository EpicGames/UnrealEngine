// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNETypes.h"

namespace UE::NNERuntimeORT::Private
{
	class FTensor
	{
	private:
		NNE::FTensorShape	Shape;
		uint64				DataSize = 0;

		FTensor() = default;

	public:
		const NNE::FTensorShape& GetShape() const
		{
			return Shape;
		}

		uint64 GetDataSize() const
		{
			return DataSize;
		}

		static FTensor Make(const NNE::FTensorShape& InShape, ENNETensorDataType DataType)
		{
			check(InShape.Volume() <= TNumericLimits<uint32>::Max());

			FTensor Tensor;
			Tensor.Shape = InShape;
			Tensor.DataSize = (uint64)NNE::GetTensorDataTypeSizeInBytes(DataType) * InShape.Volume();
			return Tensor;
		}

		static FTensor MakeFromSymbolicDesc(const NNE::FTensorDesc& TensorDesc)
		{
			return Make(NNE::FTensorShape::MakeFromSymbolic(TensorDesc.GetShape()), TensorDesc.GetDataType());
		}
	};

} // namespace UE::NNERuntimeORT::Private