// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeORTUtils.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "NNERuntimeORT.h"
#include "NNERuntimeORTEnv.h"

#if PLATFORM_WINDOWS
#include <dxcore_interface.h>
#include <dxcore.h>
#include "ID3D12DynamicRHI.h"
#endif // PLATFORM_WINDOWS

static int32 ORTProfilingSessionNumber = 0;
static TAutoConsoleVariable<bool> CVarNNERuntimeORTEnableProfiling(
	TEXT("nne.ort.enableprofiling"),
	false,
	TEXT("True if NNERuntimeORT plugin should create ORT sessions with profiling enabled.\n")
	TEXT("When profiling is enabled ORT will create standard performance tracing json files next to the editor executable.\n")
	TEXT("The files will be prefixed by 'NNERuntimeORTProfile_' and can be loaded for example using chrome://tracing.\n")
	TEXT("More information can be found at https://onnxruntime.ai/docs/performance/tune-performance/profiling-tools.html\n"),
	ECVF_Default);

namespace UE::NNERuntimeORT::Private
{

#if PLATFORM_WINDOWS
Microsoft::WRL::ComPtr<ID3D12Device1> CreateD3D12Device(IUnknown* AdapterPtr, D3D_FEATURE_LEVEL FeatureLevel) 
{
	using Microsoft::WRL::ComPtr;

	void* D3D12Module = FPlatformProcess::GetDllHandle(TEXT("d3d12.dll"));
	if (!D3D12Module)
	{
		UE_LOGF(LogNNERuntimeORT, Log, "Failed to load module 'd3d12.dll'");
		return {};
	}

	decltype(&D3D12CreateDevice) D3D12CreateDeviceFun = reinterpret_cast<decltype(&D3D12CreateDevice)>(FPlatformProcess::GetDllExport(D3D12Module, TEXT("D3D12CreateDevice")));
	if (!D3D12CreateDeviceFun)
	{
		UE_LOGF(LogNNERuntimeORT, Log, "Failed to get export 'D3D12CreateDevice' from module 'd3d12.dll'");
		return {};
	}

	ComPtr<ID3D12Device1> D3D12Device;
	HRESULT Hr = D3D12CreateDeviceFun(AdapterPtr, FeatureLevel, IID_PPV_ARGS(&D3D12Device));
	if (FAILED(Hr))
	{
		UE_LOGF(LogNNERuntimeORT, Log, "Failed to create D3D12 device, D3D12CreateDevice error code :%x", Hr);
		return {};
	}

	return D3D12Device;
}
#endif // PLATFORM_WINDOWS

bool IsRHID3D12Available()
{
#if PLATFORM_WINDOWS
	return IsRHID3D12();
#else
	return false;
#endif // PLATFORM_WINDOWS
}

// Check for DirectX 12-compatible hardware.
// Use DXGI to enumerate adapters and try to create a d3d12 device using the default adapter
bool IsD3D12Available()
{
#if PLATFORM_WINDOWS
	using Microsoft::WRL::ComPtr;

	const int32 DeviceIndex = 0;

	void* DxCoreModule = FPlatformProcess::GetDllHandle(TEXT("DXCore.dll"));
	if (!DxCoreModule)
	{
		return false;
	}

	using DXCoreCreateAdapterFactoryFunType = HRESULT __stdcall(REFIID, void**);

	DXCoreCreateAdapterFactoryFunType* DxCoreCreateAdapterFactoryFun = reinterpret_cast<DXCoreCreateAdapterFactoryFunType*>(FPlatformProcess::GetDllExport(DxCoreModule, TEXT("DXCoreCreateAdapterFactory")));
	if (!DxCoreCreateAdapterFactoryFun)
	{
		return false;
	}

	ComPtr<IDXCoreAdapterFactory> Factory;
	HRESULT Hr = DxCoreCreateAdapterFactoryFun(IID_PPV_ARGS(&Factory));
	if (FAILED(Hr))
	{
		return false;
	}

	const GUID DxGUIDs[] = { DXCORE_ADAPTER_ATTRIBUTE_D3D12_CORE_COMPUTE };

	ComPtr<IDXCoreAdapterList> AdapterList;
	Hr = Factory->CreateAdapterList(ARRAYSIZE(DxGUIDs), DxGUIDs, IID_PPV_ARGS(&AdapterList));
	if (FAILED(Hr))
	{
		return false;
	}

	const uint32 AdapterCount = AdapterList->GetAdapterCount();
	if (AdapterCount <= DeviceIndex)
	{
		UE_LOGF(LogNNERuntimeORT, Warning, "Invalid device index %d. Number of available devices is %d.", DeviceIndex, AdapterCount);
		return false;
	}

	ComPtr<IDXCoreAdapter> Adapter;
	Hr = AdapterList->GetAdapter(static_cast<uint32_t>(DeviceIndex), IID_PPV_ARGS(&Adapter));
	if (FAILED(Hr))
	{
		return false;
	}

	ComPtr<ID3D12Device> Device = CreateD3D12Device(Adapter.Get(), D3D_FEATURE_LEVEL_1_0_CORE);
	if (!Device)
	{
		return false;
	}

	return true;
#else
	return false;
#endif // PLATFORM_WINDOWS
}

// For more details about ORT graph optimization checkout
// https://onnxruntime.ai/docs/performance/model-optimizations/graph-optimizations.html

struct FGraphOptimizationLevels
{
	GraphOptimizationLevel Cooking;
	GraphOptimizationLevel Offline;
	GraphOptimizationLevel Online;
};

// CPU
static constexpr FGraphOptimizationLevels OrtCpuOptimizationLevels
{
	.Cooking = GraphOptimizationLevel::ORT_ENABLE_EXTENDED,
	.Offline = GraphOptimizationLevel::ORT_DISABLE_ALL,
	.Online = GraphOptimizationLevel::ORT_ENABLE_ALL
};

// DirectML EP
// note: optimize with DirectML EP enabled, but currently an offline optimized model can not be optimized again (only DML)!
// Therefore, if one enables offline optimization, set it to ORT_ENABLE_ALL and disable any optimization in online mode (ORT_DISABLE_ALL).
//
// note: since during cooking the DirectML Execution Provider might not be available, one can not optimize at all, because with Float16 
// Cast operators would be inserted, since the optimizer prepares the model for execution on CPU (at the moment this can not be turned off)
//
// Therefore we only optimize online for now!
static constexpr FGraphOptimizationLevels OrtDmlOptimizationLevels
{
	.Cooking = GraphOptimizationLevel::ORT_DISABLE_ALL,
	.Offline = GraphOptimizationLevel::ORT_DISABLE_ALL,
	.Online = GraphOptimizationLevel::ORT_ENABLE_ALL
};

GraphOptimizationLevel GetGraphOptimizationLevel(const FGraphOptimizationLevels &OptimizationLevels, bool bIsOnline, bool bIsCooking)
{
	if (bIsOnline)
	{
		return OptimizationLevels.Online;
	}
	else
	{
		if (bIsCooking)
		{
			return OptimizationLevels.Cooking;
		}
		else
		{
			return OptimizationLevels.Offline;
		}
	}
}

namespace OrtHelper
{
TArray<uint32> GetShape(const Ort::Value& OrtTensor)
{
	OrtTensorTypeAndShapeInfo* TypeAndShapeInfoPtr = nullptr;
	size_t DimensionsCount = 0;

	Ort::ThrowOnError(Ort::GetApi().GetTensorTypeAndShape(OrtTensor, &TypeAndShapeInfoPtr));
	Ort::ThrowOnError(Ort::GetApi().GetDimensionsCount(TypeAndShapeInfoPtr, &DimensionsCount));

	TArray<int64_t> OrtShape;

	OrtShape.SetNumUninitialized(DimensionsCount);
	Ort::ThrowOnError(Ort::GetApi().GetDimensions(TypeAndShapeInfoPtr, OrtShape.GetData(), OrtShape.Num()));
	Ort::GetApi().ReleaseTensorTypeAndShapeInfo(TypeAndShapeInfoPtr);

	TArray<uint32> Result;

	Algo::Transform(OrtShape, Result, [](int64_t Value)
	{
		check(Value >= 0);
		return (uint32)Value;
	});

	return Result;
}
} // namespace OrtHelper

GraphOptimizationLevel GetGraphOptimizationLevelForCPU(bool bIsOnline, bool bIsCooking)
{
	return GetGraphOptimizationLevel(OrtCpuOptimizationLevels, bIsOnline, bIsCooking);
}

GraphOptimizationLevel GetGraphOptimizationLevelForDML(bool bIsOnline, bool bIsCooking)
{
	return GetGraphOptimizationLevel(OrtDmlOptimizationLevels, bIsOnline, bIsCooking);
}

TUniquePtr<Ort::SessionOptions> CreateSessionOptionsDefault(const TSharedRef<FEnvironment> &Environment)
{
	const FEnvironment::FConfig Config = Environment->GetConfig();

	TUniquePtr<Ort::SessionOptions> SessionOptions = MakeUnique<Ort::SessionOptions>();

	// Configure Threading
	if (Config.bUseGlobalThreadPool)
	{
		SessionOptions->DisablePerSessionThreads();
	}
	else
	{
		SessionOptions->SetIntraOpNumThreads(Config.IntraOpNumThreads);
		SessionOptions->SetInterOpNumThreads(Config.InterOpNumThreads);
	}

	// Configure Profiling
	// Note: can be called on game or render thread
	if (CVarNNERuntimeORTEnableProfiling.GetValueOnAnyThread())
	{
		FString ProfilingFilePrefix("NNERuntimeORTProfile_");
		ProfilingFilePrefix += FString::FromInt(ORTProfilingSessionNumber);
		++ORTProfilingSessionNumber;
		#if PLATFORM_WINDOWS
			SessionOptions->EnableProfiling(*ProfilingFilePrefix);
		#else
			SessionOptions->EnableProfiling(TCHAR_TO_ANSI(*ProfilingFilePrefix));
		#endif
	}

	return SessionOptions;
}

TUniquePtr<Ort::SessionOptions> CreateSessionOptionsForDirectML(const TSharedRef<FEnvironment> &Environment, bool bRHID3D12Required)
{
#if PLATFORM_WINDOWS
	using Microsoft::WRL::ComPtr;

	TUniquePtr<Ort::SessionOptions> SessionOptions = CreateSessionOptionsDefault(Environment);
	if (!SessionOptions.IsValid())
	{
		return {};
	}

	// Configure for DirectML
	SessionOptions->SetExecutionMode(ExecutionMode::ORT_SEQUENTIAL);
	SessionOptions->DisableMemPattern();

	if (!bRHID3D12Required && !IsRHID3D12())
	{
		const int32 DeviceIndex = 0;

		const OrtDmlApi* DmlApi = nullptr;
		Ort::ThrowOnError(Ort::GetApi().GetExecutionProviderApi("DML", ORT_API_VERSION, reinterpret_cast<const void**>(&DmlApi)));
		if (!DmlApi)
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Ort DirectML Api not available!");
			return {};
		}

		OrtStatusPtr Status = DmlApi->SessionOptionsAppendExecutionProvider_DML(*SessionOptions.Get(), DeviceIndex);
		if (Status)
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Failed to add DirectML execution provider to OnnxRuntime session options: %ls", ANSI_TO_TCHAR(Ort::GetApi().GetErrorMessage(Status)));
			return {};
		}

		return SessionOptions;
	}

	if (!GDynamicRHI)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "No RHI found, could not initialize");
		return {};
	}

	// In order to use DirectML we need D3D12
	ID3D12DynamicRHI* RHI = nullptr;
	if (IsRHID3D12())
	{
		RHI = GetID3D12DynamicRHI();
	}
	else
	{
		if (GDynamicRHI)
		{
			UE_LOGF(LogNNERuntimeORT, Error, "%ls RHI is not supported by DirectML, please use D3D12.", GDynamicRHI->GetName());
			return {};
		}
		else
		{
			UE_LOGF(LogNNERuntimeORT, Error, "No RHI found");
			return {};
		}
	}

	check(RHI);

	const int32 DeviceIndex = 0;

	ID3D12Device* D3D12Device = RHI->RHIGetDevice(DeviceIndex);
	if (!D3D12Device)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Failed to get D3D12 Device from RHI for device index %d", DeviceIndex);
		return {};
	}

	DML_CREATE_DEVICE_FLAGS DmlCreateFlags = DML_CREATE_DEVICE_FLAG_NONE;

	// Set debugging flags
	if (GRHIGlobals.IsDebugLayerEnabled)
	{
		DmlCreateFlags |= DML_CREATE_DEVICE_FLAG_DEBUG;
	}

	ComPtr<IDMLDevice> DmlDevice;
	HRESULT Hr = DMLCreateDevice(D3D12Device, DmlCreateFlags, IID_PPV_ARGS(&DmlDevice));
	if (FAILED(Hr))
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Failed to create DirectML device, DMLCreateDevice error code :%x", Hr);
		return {};
	}

	const OrtDmlApi* DmlApi = nullptr;
	Ort::ThrowOnError(Ort::GetApi().GetExecutionProviderApi("DML", ORT_API_VERSION, reinterpret_cast<const void**>(&DmlApi)));
	if (!DmlApi)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Ort DirectML Api not available!");
		return {};
	}

	OrtStatusPtr Status = DmlApi->SessionOptionsAppendExecutionProvider_DML1(*SessionOptions, DmlDevice.Get(), RHI->RHIGetCommandQueue());
	if (Status)
	{
		UE_LOGF(LogNNERuntimeORT, Error, "Failed to add DirectML execution provider to OnnxRuntime session options: %ls", ANSI_TO_TCHAR(Ort::GetApi().GetErrorMessage(Status)));
		return {};
	}

	return SessionOptions;
#else
	return {};
#endif // PLATFORM_WINDOWS
}

bool OptimizeModel(const TSharedRef<FEnvironment> &Environment, Ort::SessionOptions &SessionOptions, 
					TConstArrayView64<uint8>& InputModel, TArray64<uint8>& OptimizedModel)
{
	SCOPED_NAMED_EVENT_TEXT("OrtHelper::OptimizeModel", FColor::Magenta);

	FString ProjIntermediateDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir());
	FString ModelOptimizedPath = FPaths::CreateTempFilename(*ProjIntermediateDir, TEXT("ORTOptimizerPass_Optimized"), TEXT(".onnx"));

#if PLATFORM_WINDOWS
	SessionOptions.SetOptimizedModelFilePath(*ModelOptimizedPath);
#else
	SessionOptions.SetOptimizedModelFilePath(StringCast<ANSICHAR>(*ModelOptimizedPath).Get());
#endif // PLATFORM_WINDOWS

	{
		TUniquePtr<Ort::Session> Session = CreateOrtSessionFromArray(Environment.Get(), InputModel, SessionOptions);
		if (!Session)
		{
			UE_LOGF(LogNNERuntimeORT, Error, "Failed to create ONNX Runtime session");
			
			IFileManager::Get().Delete(*ModelOptimizedPath);
			
			return false;
		}
	}

	FFileHelper::LoadFileToArray(OptimizedModel, *ModelOptimizedPath);

	IFileManager::Get().Delete(*ModelOptimizedPath);

	return true;
}

TypeInfoORT TranslateTensorTypeORTToNNE(ONNXTensorElementDataType OrtDataType)
{
	ENNETensorDataType DataType = ENNETensorDataType::None;
	uint64 ElementSize = 0;

	switch (OrtDataType) {
	case ONNX_TENSOR_ELEMENT_DATA_TYPE_UNDEFINED:
		DataType = ENNETensorDataType::None;
		ElementSize = 0;
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_FLOAT:
		DataType = ENNETensorDataType::Float;
		ElementSize = sizeof(float);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_UINT8:
		DataType = ENNETensorDataType::UInt8;
		ElementSize = sizeof(uint8);
		break;
	case ONNX_TENSOR_ELEMENT_DATA_TYPE_INT8:
		DataType = ENNETensorDataType::Int8;
		ElementSize = sizeof(int8);
		break;
	case ONNX_TENSOR_ELEMENT_DATA_TYPE_UINT16:
		DataType = ENNETensorDataType::UInt16;
		ElementSize = sizeof(uint16);
		break;
	case ONNX_TENSOR_ELEMENT_DATA_TYPE_INT16:
		DataType = ENNETensorDataType::Int16;
		ElementSize = sizeof(int16);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_INT32:
		DataType = ENNETensorDataType::Int32;
		ElementSize = sizeof(int32);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_INT64:
		DataType = ENNETensorDataType::Int64;
		ElementSize = sizeof(int64);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_STRING:
		DataType = ENNETensorDataType::Char;
		ElementSize = sizeof(char);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_BOOL:
		DataType = ENNETensorDataType::Boolean;
		ElementSize = sizeof(bool);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_FLOAT16:
		DataType = ENNETensorDataType::Half;
		ElementSize = 2;
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_DOUBLE:
		DataType = ENNETensorDataType::Double;
		ElementSize = sizeof(double);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_UINT32:
		DataType = ENNETensorDataType::UInt32;
		ElementSize = sizeof(uint32);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_UINT64:
		DataType = ENNETensorDataType::UInt64;
		ElementSize = sizeof(uint64);
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_COMPLEX64:
		DataType = ENNETensorDataType::Complex64;
		ElementSize = 8;
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_COMPLEX128:
		DataType = ENNETensorDataType::Complex128;
		ElementSize = 16;
		break;

	case ONNX_TENSOR_ELEMENT_DATA_TYPE_BFLOAT16:
		DataType = ENNETensorDataType::BFloat16;
		ElementSize = 2;
		break;

	default:
		DataType = ENNETensorDataType::None;
		break;
	}

	return TypeInfoORT{ DataType, ElementSize };
}

uint64 CalcRDGBufferSizeForDirectML(uint64 DataSize)
{
	uint64 MinimumImpliedSizeInBytes = DataSize;

	// Round up to the nearest 4 bytes.
	MinimumImpliedSizeInBytes = (MinimumImpliedSizeInBytes + 3) & ~3ull;

	return MinimumImpliedSizeInBytes;
}

template<typename FunctionType, typename... ArgTypes>
bool OrtApiCallWithStatus(FunctionType &&Function, ArgTypes &&... Args)
{
	OrtStatusPtr StatusPtr = Forward<FunctionType>(Function)(Forward<ArgTypes>(Args)...);
	if (StatusPtr)
	{
		OrtErrorCode Code = Ort::GetApi().GetErrorCode(StatusPtr);
		const char* Message = Ort::GetApi().GetErrorMessage(StatusPtr);
		UE_LOGF(LogNNERuntimeORT, Error, "ONNX Runtime error %d: %s", (int32)Code, Message)
		return false;
	}
	return true;
}

#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
template<typename FunctionType, typename... ArgTypes>
bool GuardedOrtApiCallWithStatus(FunctionType &&Function, ArgTypes &&... Args)
{
	if (FPlatformMisc::IsDebuggerPresent())
	{
		return OrtApiCallWithStatus(Forward<FunctionType>(Function), Forward<ArgTypes>(Args)...);
	}
	else
	{
		__try
		{
			return OrtApiCallWithStatus(Forward<FunctionType>(Function), Forward<ArgTypes>(Args)...);
		}
		__except(EXCEPTION_EXECUTE_HANDLER)
		{
			UE_LOGF(LogNNERuntimeORT, Error, "ONNX Runtime unknown exception (SEH)!");
			return false;
		}
	}
}
#endif // PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED

TUniquePtr<Ort::Session> CreateOrtSessionFromArray(const FEnvironment& Environment, TConstArrayView64<uint8> ModelBuffer, const Ort::SessionOptions& SessionOptions)
{
	OrtSession* SessionPtr = nullptr;
#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
	bool Success = GuardedOrtApiCallWithStatus(Ort::GetApi().CreateSessionFromArray, Environment.GetOrtEnv(), ModelBuffer.GetData(), ModelBuffer.Num(), SessionOptions, &SessionPtr);
#else
	bool Success = OrtApiCallWithStatus(Ort::GetApi().CreateSessionFromArray, Environment.GetOrtEnv(), ModelBuffer.GetData(), ModelBuffer.Num(), SessionOptions, &SessionPtr);
#endif // PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
	if (!Success)
	{
		return {};
	}

	check(SessionPtr);

	TUniquePtr<Ort::Session> Session;
	Session.Reset(static_cast<Ort::Session *>(new Ort::Session::Base(SessionPtr)));

	return Session;
}

TUniquePtr<Ort::Session> CreateOrtSession(const FEnvironment& Environment, const FString& ModelPath, const Ort::SessionOptions& SessionOptions)
{
	OrtSession* SessionPtr = nullptr;
#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
	bool Success = GuardedOrtApiCallWithStatus(Ort::GetApi().CreateSession, Environment.GetOrtEnv(), *ModelPath, SessionOptions, &SessionPtr);
#elif PLATFORM_WINDOWS
	bool Success = OrtApiCallWithStatus(Ort::GetApi().CreateSession, Environment.GetOrtEnv(), *ModelPath, SessionOptions, &SessionPtr);
#else
	bool Success = OrtApiCallWithStatus(Ort::GetApi().CreateSession, Environment.GetOrtEnv(), StringCast<ANSICHAR>(*ModelPath).Get(), SessionOptions, &SessionPtr);
#endif
	if (!Success)
	{
		return {};
	}

	check(SessionPtr);
	
	TUniquePtr<Ort::Session> Session;
	Session.Reset(static_cast<Ort::Session *>(new Ort::Session::Base(SessionPtr)));

	return Session;
}

} // namespace UE::NNERuntimeORT::Private