// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "NNEOnnxruntime.h"
#include "NNETypes.h"
#include "Templates/UniquePtr.h"

namespace UE::NNERuntimeORT::Private
{

	class FEnvironment;

	bool IsRHID3D12Available();

	bool IsD3D12Available();

namespace OrtHelper
{
	TArray<uint32> GetShape(const Ort::Value& OrtTensor);
} // namespace OrtHelper

	GraphOptimizationLevel GetGraphOptimizationLevelForCPU(bool bIsOnline, bool bIsCooking = false);

	GraphOptimizationLevel GetGraphOptimizationLevelForDML(bool bIsOnline, bool bIsCooking = false);

	TUniquePtr<Ort::SessionOptions> CreateSessionOptionsDefault(const TSharedRef<FEnvironment> &Environment);

	TUniquePtr<Ort::SessionOptions> CreateSessionOptionsForDirectML(const TSharedRef<FEnvironment> &Environment, bool bRHID3D12Required = true);

	bool OptimizeModel(const TSharedRef<FEnvironment>& Environment, Ort::SessionOptions& SessionOptions, 
		TConstArrayView64<uint8>& InputModel, TArray64<uint8>& OptimizedModel);

	struct TypeInfoORT
	{
		ENNETensorDataType DataType = ENNETensorDataType::None;
		uint64 ElementSize = 0;
	};

	TypeInfoORT TranslateTensorTypeORTToNNE(ONNXTensorElementDataType OrtDataType);

	uint64 CalcRDGBufferSizeForDirectML(uint64 DataSize);

	TUniquePtr<Ort::Session> CreateOrtSessionFromArray(const FEnvironment& Environment, TConstArrayView64<uint8> ModelBuffer, const Ort::SessionOptions& SessionOptions);
	TUniquePtr<Ort::Session> CreateOrtSession(const FEnvironment& Environment, const FString& ModelPath, const Ort::SessionOptions& SessionOptions);

} // UE::NNERuntimeORT::Private